// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputEventVisualizer.h"

#if WITH_SLATE_DEBUGGING

#include "Animation/CurveHandle.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GenericPlatform/ICursor.h"
#include "Input/Events.h"
#include "Rendering/DrawElements.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "InputEventVisualizer"

namespace InputEventVisualizer
{
	static const float PingScaleAmount = 3.0f;
	static const FName CursorPingBrush("DemoRecording.CursorPing");
}

FInputEventVisualizer::FInputEventVisualizer()
	: bShowMouseEvent(false)
	, ClickFadeTime(0.5f)
	, bShowKeyEvent(false)
	, KeyFadeTime(2.f)
	, KeyEventTime(-1.0)
	, MaxNumberOfKeys(10)
	, DemoModeMouseConsoleVariable(
		TEXT("Slate.DemoMode.MouseEvent"),
		bShowMouseEvent,
		TEXT("Visualize the cursor events for demo-recording purposes.")
		, FConsoleVariableDelegate::CreateRaw(this, &FInputEventVisualizer::HandleDemoModeEnabledChanged))
	, DemoModeKeyConsoleVariable(
		TEXT("Slate.DemoMode.KeyEvent"),
		bShowKeyEvent,
		TEXT("Visualize any pressed keys for demo-recording purposes.")
		, FConsoleVariableDelegate::CreateRaw(this, &FInputEventVisualizer::HandleDemoModeEnabledChanged))
{
	MouseKeyColorsMap.Add(EKeys::LeftMouseButton, FLinearColor(1.f, 0.f, 1.f));
	MouseKeyColorsMap.Add(EKeys::RightMouseButton, FLinearColor(1.f, 1.f, 0.f));
	MouseKeyColorsMap.Add(EKeys::MiddleMouseButton, FLinearColor(0.f, 1.f, 1.f));
	MouseKeyColorsMap.Add(EKeys::ThumbMouseButton, FLinearColor(0.33f, 0.33f, 0.33f));
	MouseKeyColorsMap.Add(EKeys::ThumbMouseButton2, FLinearColor(0.66f, 0.66f, 0.66f));
}

FInputEventVisualizer::~FInputEventVisualizer()
{
	FCoreDelegates::OnBeginFrame.RemoveAll(this);
	FSlateDebugging::UnregisterWidgetInputRoutingEvent(this);
	FSlateDebugging::PaintDebugElements.RemoveAll(this);
}

void FInputEventVisualizer::HandleDemoModeEnabledChanged(IConsoleVariable* CVar)
{
	UpdateNotificationItem();
}

void FInputEventVisualizer::HandleBeginFrameCallback()
{
	const double CurrentTime = FSlateApplication::Get().GetCurrentTime();

	if (MouseEvents.Num() > 0)
	{
		// We draw in order we received the events, to draw over older events
		const double ClickFadeTimeAsDouble = static_cast<double>(ClickFadeTime);
		MouseEvents.RemoveAll([CurrentTime, ClickFadeTimeAsDouble](const FMouseEventInfo& Event)
		{
			return CurrentTime - Event.EventTime > ClickFadeTimeAsDouble;
		});
	}

	if (KeyEvents.Num() > 0)
	{
		if (CurrentTime - KeyEventTime > static_cast<double>(KeyFadeTime))
		{
			KeyEvents.Reset();
			UpdateKeyText();
		}
	}
}

void FInputEventVisualizer::UpdateNotificationItem()
{
	FCoreDelegates::OnBeginFrame.RemoveAll(this);
	FSlateDebugging::UnregisterWidgetInputRoutingEvent(this);
	FSlateDebugging::PaintDebugElements.RemoveAll(this);

	if (bShowKeyEvent || bShowMouseEvent)
	{
		FCoreDelegates::OnBeginFrame.AddRaw(this, &FInputEventVisualizer::HandleBeginFrameCallback);
		FSlateDebugging::RegisterWidgetInputRoutingEvent(this);
	}

	if (bShowMouseEvent)
	{
		FSlateDebugging::PaintDebugElements.AddRaw(this, &FInputEventVisualizer::HandlePaint);
	}
	else
	{
		MouseEvents.Reset();
	}

	if (bShowKeyEvent)
	{	
		TSharedPtr<SNotificationItem> OwningNotificationPinned = WeakOwningNotification.Pin();
		if (OwningNotificationPinned == nullptr)
		{
			FNotificationInfo Info{ FText::GetEmpty() };
			Info.Image = FCoreStyle::Get().GetBrush("NotificationList.ItemBackground");
			Info.bUseThrobber = false;
			Info.bUseSuccessFailIcons = false;
			Info.bUseLargeFont = false;
			Info.bFireAndForget = false;
			Info.FadeOutDuration = 0.5f;
	
			WeakOwningNotification = FSlateNotificationManager::Get().AddNotification(Info);
		}
		UpdateKeyText();
	}
	else
	{
		KeyEvents.Reset();
		if (TSharedPtr<SNotificationItem> OwningNotificationPinned = WeakOwningNotification.Pin())
		{
			OwningNotificationPinned->Fadeout();
		}
		WeakOwningNotification.Reset();
	}
}

void FInputEventVisualizer::UpdateKeyText()
{
	if (TSharedPtr<SNotificationItem> OwningNotificationPinned = WeakOwningNotification.Pin())
	{
		TArray<FText> KeyNames;
		KeyNames.Reserve(KeyEvents.Num());
		FTextBuilder Builder;
		FText Delimiter = LOCTEXT("KeySeparator", "  ");
		for (const FKeyEventInfo& KeyEventInfo : KeyEvents)
		{
			FText KeyStateText = FText::GetEmpty();
			if (KeyEventInfo.KeysState.IsCommandDown())
			{
				KeyStateText = LOCTEXT("CommandKey", "Ctrl");
			}
			else if (KeyEventInfo.KeysState.IsAltDown())
			{
				KeyStateText = LOCTEXT("AltKey", "Alt");
			}
			else if (KeyEventInfo.KeysState.IsShiftDown())
			{
				KeyStateText = LOCTEXT("ShiftKey", "Shift");
			}

			if (!KeyStateText.IsEmpty())
			{
				KeyNames.Add(FText::Format(LOCTEXT("KeyStateFormat", "{0}-{1}"), KeyStateText, KeyEventInfo.Key.GetDisplayName(false)));
			}
			else
			{
				KeyNames.Add(KeyEventInfo.Key.GetDisplayName(false));
			}
		}
		OwningNotificationPinned->SetText(FText::Join(Delimiter, KeyNames));
	}
}

void FInputEventVisualizer::OnProcessInput(ESlateDebuggingInputEvent InputEventType, const FInputEvent& Event)
{
	if (bShowMouseEvent && Event.IsPointerEvent())
	{
		const FPointerEvent& PointerEvent = static_cast<const FPointerEvent&>(Event);
		const FKey Key = PointerEvent.GetEffectingButton();
		if (Key.IsMouseButton())
		{
			FMouseEventInfo* FoundMouseEvent = MouseEvents.FindByPredicate([Key](const FMouseEventInfo& EventInfo) { return EventInfo.Key == Key; });
			if (FoundMouseEvent == nullptr)
			{
				FoundMouseEvent = &MouseEvents.AddDefaulted_GetRef();
				FoundMouseEvent->Key = Key;
				if (FLinearColor* FoundColor = MouseKeyColorsMap.Find(Key))
				{
					FoundMouseEvent->PingColor = *FoundColor;
				}
				else
				{
					FoundMouseEvent->PingColor = FLinearColor(1.f, 1.f, 1.f);
				}
			}
			FoundMouseEvent->EventTime = FSlateApplication::Get().GetCurrentTime();
			FoundMouseEvent->CursorPingPosition = PointerEvent.GetScreenSpacePosition();
		}
	}
	else if (bShowKeyEvent && Event.IsKeyEvent() && InputEventType == ESlateDebuggingInputEvent::KeyDown)
	{
		const FKeyEvent& KeyEvent = static_cast<const FKeyEvent&>(Event);
		if (KeyEvents.Num() >= MaxNumberOfKeys)
		{
			KeyEvents.RemoveAt(0, KeyEvents.Num() - MaxNumberOfKeys + 1);
		}
		KeyEvents.Emplace(KeyEvent.GetKey(), KeyEvent.GetModifierKeys());
		KeyEventTime = FSlateApplication::Get().GetCurrentTime();

		UpdateKeyText();
	}
}

void FInputEventVisualizer::HandlePaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, int32& OutLayerId)
{
	ensure(bShowMouseEvent);

	const SWindow* WindowBeingDrawn = OutDrawElements.GetPaintWindow();
	TSharedPtr<ICursor> PlatformCursor = FSlateApplication::Get().GetPlatformCursor();
	if (WindowBeingDrawn && PlatformCursor.IsValid() && PlatformCursor->GetType() != EMouseCursor::None)
	{
		const double CurrentTime = FSlateApplication::Get().GetCurrentTime();

		if (MouseEvents.Num() > 0)
		{
			// We draw in order we received the events, to draw over older events

			for (int32 Index = MouseEvents.Num() - 1; Index >= 0; --Index)
			{
				if (CurrentTime - MouseEvents[Index].EventTime > ClickFadeTime)
				{
					MouseEvents.RemoveAt(Index);
				}
			}

			int32 NewLayerId = OutLayerId++;
			const FVector2f CursorSize = FSlateApplication::Get().GetCursorSize();
			const FSlateBrush* Brush = FCoreStyle::Get().GetBrush(InputEventVisualizer::CursorPingBrush);
			for (const FMouseEventInfo& Event : MouseEvents)
			{
				// Normalized animation value for the cursor ping between 0 and 1.
				const float AnimAmount = (float)(CurrentTime - Event.EventTime) / ClickFadeTime;

				const FVector2f CursorPosDesktopSpace = Event.CursorPingPosition;
				const FVector2f PingSize = CursorSize * InputEventVisualizer::PingScaleAmount * FCurveHandle::ApplyEasing(AnimAmount, ECurveEaseFunction::QuadOut);
				FLinearColor PingColor = Event.PingColor;
				PingColor.A = 1.0f - FCurveHandle::ApplyEasing(AnimAmount, ECurveEaseFunction::QuadIn);

				FGeometry CursorHighlightGeometry = FGeometry::MakeRoot(PingSize, FSlateLayoutTransform(CursorPosDesktopSpace - PingSize / 2));
				CursorHighlightGeometry.AppendTransform(Inverse(WindowBeingDrawn->GetLocalToScreenTransform()));
				CursorHighlightGeometry.AppendTransform(FSlateLayoutTransform(WindowBeingDrawn->GetDPIScaleFactor() * FSlateApplicationBase::Get().GetApplicationScale(), FVector2f::ZeroVector));

				FSlateDrawElement::MakeBox(
					OutDrawElements,
					NewLayerId,
					CursorHighlightGeometry.ToPaintGeometry(),
					Brush,
					ESlateDrawEffect::None,
					PingColor
				);
			}
		}
	}
}

void FInputEventVisualizer::PopulateMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("MouseButtonLabel", "Mouse Click"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FInputEventVisualizer::HandleToggleMouseEvent),
			FCanExecuteAction(),
			FGetActionCheckState::CreateRaw(this, &FInputEventVisualizer::HandleGetMouseEventCheckState)),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);


	MenuBuilder.AddMenuEntry(
		LOCTEXT("KeyLabel", "Key"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FInputEventVisualizer::HandleToggleKeyEvent),
			FCanExecuteAction(),
			FGetActionCheckState::CreateRaw(this, &FInputEventVisualizer::HandleGetKeyEventCheckState)),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);
}

void FInputEventVisualizer::HandleToggleMouseEvent()
{
	bShowMouseEvent = !bShowMouseEvent;
	UpdateNotificationItem();
}

ECheckBoxState FInputEventVisualizer::HandleGetMouseEventCheckState() const
{
	return bShowMouseEvent ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FInputEventVisualizer::HandleToggleKeyEvent()
{
	bShowKeyEvent = !bShowKeyEvent;
	UpdateNotificationItem();
}

ECheckBoxState FInputEventVisualizer::HandleGetKeyEventCheckState() const
{
	return bShowKeyEvent ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

#undef LOCTEXT_NAMESPACE

#endif //WITH_SLATE_DEBUGGING