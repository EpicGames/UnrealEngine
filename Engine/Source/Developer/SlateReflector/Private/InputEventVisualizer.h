// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Debugging/SlateDebugging.h"
#include "HAL/IConsoleManager.h"

#if WITH_SLATE_DEBUGGING

struct FInputEvent;
class FSlateWindowElementList;
class FMenuBuilder;
class SNotificationItem;
enum class ECheckBoxState : uint8;

/**
 * 
 */
class FInputEventVisualizer : public FSlateDebugging::IWidgetInputRoutingEvent
{
public:
	FInputEventVisualizer();
	virtual ~FInputEventVisualizer();

	void PopulateMenu(FMenuBuilder& MenuBuilder);

	//~ Begin IWidgetInputRoutingEvent interface
	virtual void OnProcessInput(ESlateDebuggingInputEvent InputEventType, const FInputEvent& Event) override;
	virtual void OnPreProcessInput(ESlateDebuggingInputEvent InputEventType, const TCHAR* InputPrecessorName, bool bHandled) override {}
	virtual void OnRouteInput(ESlateDebuggingInputEvent InputEventType, const FName& RoutedType) override {}
	virtual void OnInputEvent(ESlateDebuggingInputEvent InputEventType, const FReply& InReply, const TSharedPtr<SWidget>& HandlerWidget) override {}
	virtual void OnInputRouted(ESlateDebuggingInputEvent InputEventType) override {}
	virtual void OnInputProcessed(ESlateDebuggingInputEvent InputEventType) override {}
	//~ End IWidgetInputRoutingEvent interface
	
private:
	void HandlePaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, FSlateWindowElementList& InOutDrawElements, int32& InOutLayerId);

	void HandleDemoModeEnabledChanged(IConsoleVariable* CVar);
	void HandleBeginFrameCallback();
	void HandleEndWindow(const FSlateWindowElementList& ElementList);

	void UpdateNotificationItem();
	void UpdateKeyText();

	void HandleToggleMouseEvent();
	ECheckBoxState HandleGetMouseEventCheckState() const;
	void HandleToggleKeyEvent();
	ECheckBoxState HandleGetKeyEventCheckState() const;

private:
	struct FMouseEventInfo
	{
		FKey Key;
		double EventTime;
		FVector2f CursorPingPosition;
		FLinearColor PingColor;
	};
	
	struct FKeyEventInfo
	{
		FKey Key;
		FModifierKeysState KeysState;
		FKeyEventInfo() = default;
		FKeyEventInfo(const FKey& InKey, const FModifierKeysState& InKeysState)
			: Key(InKey), KeysState(InKeysState)
		{}
	};

	bool bShowMouseEvent;
	float ClickFadeTime;
	TArray<FMouseEventInfo> MouseEvents;
	TMap<FKey, FLinearColor> MouseKeyColorsMap;

	bool bShowKeyEvent;
	float KeyFadeTime;
	double KeyEventTime;
	int32 MaxNumberOfKeys;
	TArray<FKeyEventInfo> KeyEvents;
	TWeakPtr<SNotificationItem> WeakOwningNotification;

	FAutoConsoleVariableRef DemoModeMouseConsoleVariable;
	FAutoConsoleVariableRef DemoModeKeyConsoleVariable;
};

#else

class FMenuBuilder;

class FInputEventVisualizer
{
};

#endif //WITH_SLATE_DEBUGGING