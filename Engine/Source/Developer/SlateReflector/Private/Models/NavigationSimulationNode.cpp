// Copyright Epic Games, Inc. All Rights Reserved.

#include "Models/NavigationSimulationNode.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Layout/WidgetPath.h"
#include "Models/WidgetReflectorNode.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "NavigationSimulationNode"

/**
 * -----------------------------------------------------------------------------
 * FNavigationSimulationWidgetInfo
 * -----------------------------------------------------------------------------
 */
FNavigationSimulationWidgetInfo::FNavigationSimulationWidgetInfo(const TSharedPtr<const SWidget>& Widget)
	: WidgetPtr(FWidgetReflectorNodeUtils::GetWidgetAddress(Widget))
	, WidgetLive(Widget)
	, WidgetTypeAndShortName(FWidgetReflectorNodeUtils::GetWidgetTypeAndShortName(Widget))
	, bHasGeometry(false)
{

}

FNavigationSimulationWidgetInfo::FNavigationSimulationWidgetInfo(const FWidgetPath& WidgetPath)
	: WidgetPtr(0)
	, bHasGeometry(WidgetPath.IsValid())
{
	if (bHasGeometry)
	{
		WidgetLive = WidgetPath.GetLastWidget();
		WidgetPtr = FWidgetReflectorNodeUtils::GetWidgetAddress(WidgetPath.GetLastWidget());
		WidgetTypeAndShortName = FWidgetReflectorNodeUtils::GetWidgetTypeAndShortName(WidgetPath.GetLastWidget());
		WidgetGeometry = WidgetPath.Widgets.Last().Geometry;
	}
}

FNavigationSimulationWidgetNodeItem::FNavigationSimulationWidgetNodeItem(const FSlateNavigationEventSimulator::FSimulationResult& Result)
	: NavigationType(Result.NavigationType)
	, Destination(Result.NavigationDestination)
	, ReplyEventHandler(Result.NavigationReply.EventHandler)
	, ReplyFocusRecipient(Result.NavigationReply.FocusRecipient)
	, ReplyBoundaryRule(Result.NavigationReply.BoundaryRule)
	, RoutedReason(Result.RoutedReason)
	, WidgetThatShouldReceivedFocus(Result.WidgetThatShouldReceivedFocus)
	, FocusedWidget(Result.FocusedWidgetPath)
	, bIsDynamic(Result.bIsDynamic)
	, bAlwaysHandleNavigationAttempt(Result.bAlwaysHandleNavigationAttempt)
	, bCanFindWidgetForSetFocus(Result.bCanFindWidgetForSetFocus)
	, bRoutedHandlerHasNavigationMeta(Result.bRoutedHandlerHasNavigationMeta)
	, bHandledByViewport(Result.bHandledByViewport)
{

}

void FNavigationSimulationWidgetNodeItem::ResetLiveWidget()
{
	Destination.WidgetLive.Reset();
	ReplyEventHandler.WidgetLive.Reset();
	ReplyFocusRecipient.WidgetLive.Reset();
	WidgetThatShouldReceivedFocus.WidgetLive.Reset();
	FocusedWidget.WidgetLive.Reset();
}

FNavigationSimulationWidgetNode::FNavigationSimulationWidgetNode(ENavigationSimulationNodeType NodeType, const FWidgetPath& InNavigationSource)
	: NavigationSource(InNavigationSource)
	, NodeType(NodeType)
{
	if (NodeType == ENavigationSimulationNodeType::Snapshot)
	{
		NavigationSource.WidgetLive.Reset();
	}
}

namespace NavigationSimulationNodeUtilsInternal
{
	TArray<FNavigationSimulationWidgetNodePtr> BuildNavigationSimulationNodeList(ENavigationSimulationNodeType NodeType, const TArray<FSlateNavigationEventSimulator::FSimulationResult>& SimulationResult)
	{
		TMap<const SWidget*, FNavigationSimulationWidgetNodePtr> WidgetMap;
		for (const FSlateNavigationEventSimulator::FSimulationResult& Element : SimulationResult)
		{
			if (Element.IsValid())
			{
				const SWidget* ElementWidget = &Element.NavigationSource.GetLastWidget().Get();
				if (!WidgetMap.Contains(ElementWidget))
				{
					WidgetMap.Add(ElementWidget, MakeShared<FNavigationSimulationWidgetNode>(NodeType, Element.NavigationSource));
				}

				FNavigationSimulationWidgetNodeItem& NodeItem = WidgetMap[ElementWidget]->Simulations.Emplace_GetRef(Element);
				if (NodeType == ENavigationSimulationNodeType::Snapshot)
				{
					NodeItem.ResetLiveWidget();
				}
			}
		}

		TArray<FNavigationSimulationWidgetNodePtr> GeneratedArray;
		WidgetMap.GenerateValueArray(GeneratedArray);

		return MoveTemp(GeneratedArray);
	}
}

TSharedRef<FJsonValue> FNavigationSimulationWidgetNode::ToJson(const FNavigationSimulationWidgetNode& Node)
{
	struct Internal
	{
		static FString ConvertPtrIntToString(FNavigationSimulationWidgetInfo::TPointerAsInt Value)
		{
			return FWidgetReflectorNodeUtils::WidgetAddressToString(Value);
		}
		static TSharedRef<FJsonValue> CreateJsonValue(const UE::Slate::FDeprecateVector2DResult& InVec2D)
		{
			return CreateJsonValue(FVector2D(InVec2D));
		}
		static TSharedRef<FJsonValue> CreateJsonValue(const FVector2D& InVec2D)
		{
			TArray<TSharedPtr<FJsonValue>> StructJsonArray;
			StructJsonArray.Add(MakeShared<FJsonValueNumber>(InVec2D.X));
			StructJsonArray.Add(MakeShared<FJsonValueNumber>(InVec2D.Y));
			return MakeShared<FJsonValueArray>(StructJsonArray);
		}
		static TSharedRef<FJsonValue> CreateJsonValue(const FMatrix2x2& InMatrix)
		{
			float m00, m01, m10, m11;
			InMatrix.GetMatrix(m00, m01, m10, m11);

			TArray<TSharedPtr<FJsonValue>> StructJsonArray;
			StructJsonArray.Add(MakeShared<FJsonValueNumber>(m00));
			StructJsonArray.Add(MakeShared<FJsonValueNumber>(m01));
			StructJsonArray.Add(MakeShared<FJsonValueNumber>(m10));
			StructJsonArray.Add(MakeShared<FJsonValueNumber>(m11));
			return MakeShared<FJsonValueArray>(StructJsonArray);
		}
		static TSharedRef<FJsonValue> CreateJsonValue(const FSlateLayoutTransform& InLayoutTransform)
		{
			TSharedRef<FJsonObject> StructJsonObject = MakeShared<FJsonObject>();
			StructJsonObject->SetNumberField(TEXT("Scale"), InLayoutTransform.GetScale());
			StructJsonObject->SetField(TEXT("Translation"), CreateJsonValue(InLayoutTransform.GetTranslation()));
			return MakeShared<FJsonValueObject>(StructJsonObject);
		}
		static TSharedRef<FJsonValue> CreateJsonValue(const FSlateRenderTransform& InRenderTransform)
		{
			TSharedRef<FJsonObject> StructJsonObject = MakeShared<FJsonObject>();
			StructJsonObject->SetField(TEXT("Matrix"), CreateJsonValue(InRenderTransform.GetMatrix()));
			StructJsonObject->SetField(TEXT("Translation"), CreateJsonValue(InRenderTransform.GetTranslation()));
			return MakeShared<FJsonValueObject>(StructJsonObject);
		}
		static TSharedRef<FJsonValue> CreateJsonValue(const FGeometry& InGeo)
		{
			TSharedRef<FJsonObject> StructJsonObject = MakeShared<FJsonObject>();
			StructJsonObject->SetField(TEXT("AccumulatedRenderTransform"), CreateJsonValue(InGeo.GetAccumulatedRenderTransform()));
			StructJsonObject->SetField(TEXT("AccumulatedLayoutTransform"), CreateJsonValue(InGeo.GetAccumulatedLayoutTransform()));
			StructJsonObject->SetField(TEXT("LocalSize"), CreateJsonValue(InGeo.GetLocalSize()));
			return MakeShared<FJsonValueObject>(StructJsonObject);
		}
		static TSharedRef<FJsonValue> CreateJsonValue(const FNavigationSimulationWidgetInfo& Value)
		{
			TSharedRef<FJsonObject> StructJsonObject = MakeShared<FJsonObject>();
			StructJsonObject->SetStringField(TEXT("WidgetPtr"), ConvertPtrIntToString(Value.WidgetPtr));
			StructJsonObject->SetStringField(TEXT("WidgetTypeAndShortName"), Value.WidgetTypeAndShortName.ToString());
			StructJsonObject->SetBoolField(TEXT("bHasGeometry"), Value.bHasGeometry);
			if (Value.bHasGeometry)
			{
				StructJsonObject->SetField(TEXT("WidgetGeometry"), CreateJsonValue(Value.WidgetGeometry));
			}
			return MakeShared<FJsonValueObject>(StructJsonObject);
		}
		static TSharedRef<FJsonValue> CreateJsonValue(const FNavigationSimulationWidgetNodeItem& Value)
		{
			TSharedRef<FJsonObject> StructJsonObject = MakeShared<FJsonObject>();
			StructJsonObject->SetNumberField(TEXT("NavigationType"), (int32)Value.NavigationType);
			StructJsonObject->SetNumberField(TEXT("ReplyBoundaryRule"), (int32)Value.ReplyBoundaryRule);
			StructJsonObject->SetNumberField(TEXT("RoutedReason"), (int32)Value.RoutedReason);
			StructJsonObject->SetField(TEXT("Destination"), CreateJsonValue(Value.Destination));
			StructJsonObject->SetField(TEXT("ReplyEventHandler"), CreateJsonValue(Value.ReplyEventHandler));
			StructJsonObject->SetField(TEXT("ReplyFocusRecipient"), CreateJsonValue(Value.ReplyFocusRecipient));
			StructJsonObject->SetField(TEXT("WidgetThatShouldReceivedFocus"), CreateJsonValue(Value.WidgetThatShouldReceivedFocus));
			StructJsonObject->SetField(TEXT("FocusedWidget"), CreateJsonValue(Value.FocusedWidget));
			StructJsonObject->SetBoolField(TEXT("bIsDynamic"), Value.bIsDynamic);
			StructJsonObject->SetBoolField(TEXT("bAlwaysHandleNavigationAttempt"), Value.bAlwaysHandleNavigationAttempt);
			StructJsonObject->SetBoolField(TEXT("bCanFindWidgetForSetFocus"), Value.bCanFindWidgetForSetFocus);
			StructJsonObject->SetBoolField(TEXT("bRoutedHandlerHasNavigationMeta"), Value.bRoutedHandlerHasNavigationMeta);
			StructJsonObject->SetBoolField(TEXT("bHandledByViewport"), Value.bHandledByViewport);
			return MakeShared<FJsonValueObject>(StructJsonObject);
		}
	};

	TSharedRef<FJsonObject> RootJsonObject = MakeShared<FJsonObject>();
	RootJsonObject->SetField(TEXT("NavigationSource"), Internal::CreateJsonValue(Node.NavigationSource));
	RootJsonObject->SetNumberField(TEXT("NodeType"), (int32)Node.NodeType);
	TArray<TSharedPtr<FJsonValue>> SimulationArray;
	for (const FNavigationSimulationWidgetNodeItem& Item : Node.Simulations)
	{
		SimulationArray.Add(Internal::CreateJsonValue(Item));
	}
	RootJsonObject->SetArrayField(TEXT("Simulations"), SimulationArray);
	return MakeShared<FJsonValueObject>(RootJsonObject);
}

FNavigationSimulationWidgetNodePtr FNavigationSimulationWidgetNode::FromJson(const TSharedRef<FJsonValue>& RootJsonValue)
{
	struct Internal
	{
		static void ParseJsonValue(const TSharedPtr<FJsonValue>& InJsonValue, FVector2D& Out)
		{
			if (!InJsonValue.IsValid())
			{
				return;
			}
			const TArray<TSharedPtr<FJsonValue>>& StructJsonArray = InJsonValue->AsArray();
			check(StructJsonArray.Num() == 2);

			Out = FVector2D((float)StructJsonArray[0]->AsNumber(), (float)StructJsonArray[1]->AsNumber());
		}
		static FNavigationSimulationWidgetInfo::TPointerAsInt ParsePtrIntFromString(const FString& Value)
		{
			FWidgetReflectorNodeBase::TPointerAsInt Result = 0;
			LexFromString(Result, *Value);
			return Result;
		}
		static void ParseJsonValue(const TSharedPtr<FJsonValue>& InJsonValue, FMatrix2x2& Out)
		{
			if (!InJsonValue.IsValid())
			{
				return;
			}
			const TArray<TSharedPtr<FJsonValue>>& StructJsonArray = InJsonValue->AsArray();
			check(StructJsonArray.Num() == 4);

			Out = FMatrix2x2(
				(float)StructJsonArray[0]->AsNumber(),
				(float)StructJsonArray[1]->AsNumber(),
				(float)StructJsonArray[2]->AsNumber(),
				(float)StructJsonArray[3]->AsNumber()
			);
		}
		static void ParseJsonValue(const TSharedPtr<FJsonValue>& InJsonValue, FSlateLayoutTransform& Out)
		{
			if (!InJsonValue.IsValid())
			{
				return;
			}
			const TSharedPtr<FJsonObject>& StructJsonObject = InJsonValue->AsObject();
			check(StructJsonObject.IsValid());


			float Scale = (float)StructJsonObject->GetNumberField(TEXT("Scale"));
			FVector2D Translation;
			ParseJsonValue(StructJsonObject->GetField<EJson::None>(TEXT("Translation")), Translation);
			Out = FSlateLayoutTransform(Scale, Translation);
		}
		static void ParseJsonValue(const TSharedPtr<FJsonValue>& InJsonValue, FSlateRenderTransform& Out)
		{
			if (!InJsonValue.IsValid())
			{
				return;
			}
			const TSharedPtr<FJsonObject>& StructJsonObject = InJsonValue->AsObject();
			check(StructJsonObject.IsValid());

			FMatrix2x2 Matrix;
			ParseJsonValue(StructJsonObject->GetField<EJson::None>(TEXT("Matrix")), Matrix);
			FVector2D Translation;
			ParseJsonValue(StructJsonObject->GetField<EJson::None>(TEXT("Translation")), Translation);
			Out = FSlateRenderTransform(Matrix, Translation);
		}
		static void ParseJsonValue(const TSharedPtr<FJsonValue>& InJsonValue, FGeometry& Out)
		{
			if (!InJsonValue.IsValid())
			{
				return;
			}
			const TSharedPtr<FJsonObject>& StructJsonObject = InJsonValue->AsObject();
			check(StructJsonObject.IsValid());

			FSlateLayoutTransform LayoutTransform;
			FSlateRenderTransform RenderTransform;
			ParseJsonValue(StructJsonObject->GetField<EJson::None>(TEXT("AccumulatedRenderTransform")), RenderTransform);
			ParseJsonValue(StructJsonObject->GetField<EJson::None>(TEXT("AccumulatedLayoutTransform")), LayoutTransform);
			FVector2D LocalSize;
			ParseJsonValue(StructJsonObject->GetField<EJson::None>(TEXT("LocalSize")), LocalSize);

			Out = FGeometry::MakeRoot(LocalSize, LayoutTransform, RenderTransform);
		}
		static void ParseJsonValue(const TSharedPtr<FJsonValue>& InJsonValue, FNavigationSimulationWidgetInfo& Out)
		{
			if (!InJsonValue.IsValid())
			{
				return;
			}
			const TSharedPtr<FJsonObject>& StructJsonObject = InJsonValue->AsObject();
			check(StructJsonObject.IsValid());

			Out.WidgetPtr = ParsePtrIntFromString(StructJsonObject->GetStringField(TEXT("WidgetPtr")));
			Out.WidgetTypeAndShortName = FText::FromString(StructJsonObject->GetStringField(TEXT("WidgetTypeAndShortName")));
			Out.bHasGeometry = StructJsonObject->GetBoolField(TEXT("bHasGeometry"));
			if (Out.bHasGeometry)
			{
				ParseJsonValue(StructJsonObject->GetField<EJson::None>(TEXT("WidgetGeometry")), Out.WidgetGeometry);
			}
		}
		static void ParseJsonValue(const TSharedPtr<FJsonValue>& InJsonValue, FNavigationSimulationWidgetNodeItem& Out)
		{
			if (!InJsonValue.IsValid())
			{
				return;
			}
			const TSharedPtr<FJsonObject>& StructJsonObject = InJsonValue->AsObject();
			check(StructJsonObject.IsValid());

			Out.NavigationType = (EUINavigation)(int32)StructJsonObject->GetNumberField(TEXT("NavigationType"));
			Out.ReplyBoundaryRule = (EUINavigationRule)(int32)StructJsonObject->GetNumberField(TEXT("ReplyBoundaryRule"));
			Out.RoutedReason = (FSlateNavigationEventSimulator::ERoutedReason)(int32)StructJsonObject->GetNumberField(TEXT("RoutedReason"));
			ParseJsonValue(StructJsonObject->GetField<EJson::None>(TEXT("Destination")), Out.Destination);
			ParseJsonValue(StructJsonObject->GetField<EJson::None>(TEXT("ReplyEventHandler")), Out.ReplyEventHandler);
			ParseJsonValue(StructJsonObject->GetField<EJson::None>(TEXT("ReplyFocusRecipient")), Out.ReplyFocusRecipient);
			ParseJsonValue(StructJsonObject->GetField<EJson::None>(TEXT("WidgetThatShouldReceivedFocus")), Out.WidgetThatShouldReceivedFocus);
			ParseJsonValue(StructJsonObject->GetField<EJson::None>(TEXT("FocusedWidget")), Out.FocusedWidget);
			Out.bIsDynamic = StructJsonObject->GetBoolField(TEXT("bIsDynamic"));
			Out.bAlwaysHandleNavigationAttempt = StructJsonObject->GetBoolField(TEXT("bAlwaysHandleNavigationAttempt"));
			Out.bCanFindWidgetForSetFocus = StructJsonObject->GetBoolField(TEXT("bCanFindWidgetForSetFocus"));
			Out.bRoutedHandlerHasNavigationMeta = StructJsonObject->GetBoolField(TEXT("bRoutedHandlerHasNavigationMeta"));
			Out.bHandledByViewport = StructJsonObject->GetBoolField(TEXT("bHandledByViewport"));
		}
	};

	TSharedRef<FNavigationSimulationWidgetNode> Result = MakeShared<FNavigationSimulationWidgetNode>();
	const TSharedPtr<FJsonObject>& RootJsonObject = RootJsonValue->AsObject();
	check(RootJsonObject.IsValid());
	Internal::ParseJsonValue(RootJsonObject->GetField<EJson::None>(TEXT("NavigationSource")), Result->NavigationSource);
	Result->NodeType = (ENavigationSimulationNodeType)((int32)RootJsonObject->GetNumberField(TEXT("NodeType")));
	const TArray<TSharedPtr<FJsonValue>>& SimulationsJsonArray = RootJsonObject->GetArrayField(TEXT("Simulations"));
	for (const TSharedPtr<FJsonValue>& SimulationsJsonValue : SimulationsJsonArray)
	{
		FNavigationSimulationWidgetNodeItem NodeItem;
		Internal::ParseJsonValue(SimulationsJsonValue.ToSharedRef(), NodeItem);
		Result->Simulations.Add(MoveTemp(NodeItem));
	}
	return Result;
}

TArray<FNavigationSimulationWidgetNodePtr> FNavigationSimulationNodeUtils::BuildNavigationSimulationNodeListForLive(const TArray<FSlateNavigationEventSimulator::FSimulationResult>& SimulationResult)
{
	return NavigationSimulationNodeUtilsInternal::BuildNavigationSimulationNodeList(ENavigationSimulationNodeType::Live, SimulationResult);
}

TArray<FNavigationSimulationWidgetNodePtr> FNavigationSimulationNodeUtils::BuildNavigationSimulationNodeListForSnapshot(const TArray<FSlateNavigationEventSimulator::FSimulationResult>& SimulationResult)
{
	return NavigationSimulationNodeUtilsInternal::BuildNavigationSimulationNodeList(ENavigationSimulationNodeType::Snapshot, SimulationResult);
}

TArray<FNavigationSimulationWidgetNodePtr> FNavigationSimulationNodeUtils::FindReflectorNodes(const TArray<FNavigationSimulationWidgetNodePtr>& Nodes, const TArray<TSharedRef<FWidgetReflectorNodeBase>>& ToFind)
{
	TArray<FNavigationSimulationWidgetNodePtr> Result;
	Result.Reserve(ToFind.Num());
	for (const TSharedRef<FWidgetReflectorNodeBase>& ReflectorNode : ToFind)
	{
		int32 FoundIndex = IndexOfSnapshotWidget(Nodes, ReflectorNode->GetWidgetAddress());
		if (Nodes.IsValidIndex(FoundIndex))
		{
			Result.Add(Nodes[FoundIndex]);
		}
	}
	return Result;
}

int32 FNavigationSimulationNodeUtils::IndexOfLiveWidget(const TArray<FNavigationSimulationWidgetNodePtr>& Nodes, const TSharedPtr<const SWidget>& WidgetToFind)
{
	return Nodes.IndexOfByPredicate([WidgetToFind](const FNavigationSimulationWidgetNodePtr& Node) { return Node->NavigationSource.WidgetLive == WidgetToFind; });
}

int32 FNavigationSimulationNodeUtils::IndexOfSnapshotWidget(const TArray<FNavigationSimulationWidgetNodePtr>& Nodes, FNavigationSimulationWidgetInfo::TPointerAsInt WidgetToFind)
{
	return Nodes.IndexOfByPredicate([WidgetToFind](const FNavigationSimulationWidgetNodePtr& Node) { return Node->NavigationSource.WidgetPtr == WidgetToFind; });
}

#undef LOCTEXT_NAMESPACE
