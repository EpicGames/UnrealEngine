// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Geometry.h"
#include "SlateNavigationEventSimulator.h"

class SWidget;
class FWidgetPath;
class FWidgetReflectorNodeBase;
class FJsonValue;

struct FNavigationSimulationWidgetInfo;
struct FNavigationSimulationWidgetNodeItem;
struct FNavigationSimulationWidgetNode;
using FNavigationSimulationWidgetNodePtr = TSharedPtr<FNavigationSimulationWidgetNode>;

enum class ENavigationSimulationNodeType : uint8
{
	Live,
	Snapshot,
};

struct FNavigationSimulationWidgetInfo
{
	// Should be PTRINT but if the platform that produce the file is 64bits and the reader is not then we could have a issue
	using TPointerAsInt = uint64;

	FNavigationSimulationWidgetInfo() = default;
	explicit FNavigationSimulationWidgetInfo(const TSharedPtr<const SWidget>& Widget);
	explicit FNavigationSimulationWidgetInfo(const FWidgetPath& WidgetPath);

	/** Is the widget set to null. */
	bool IsWidgetExplicitlyNull() const { return WidgetPtr == 0; }

	TPointerAsInt WidgetPtr;
	TWeakPtr<const SWidget> WidgetLive;

	FText WidgetTypeAndShortName;
	FGeometry WidgetGeometry;
	bool bHasGeometry;
};

struct FNavigationSimulationWidgetNodeItem
{
	FNavigationSimulationWidgetNodeItem() = default;
	explicit FNavigationSimulationWidgetNodeItem(const FSlateNavigationEventSimulator::FSimulationResult& Result);

	EUINavigation NavigationType;
	FNavigationSimulationWidgetInfo Destination;
	FNavigationSimulationWidgetInfo ReplyEventHandler;
	FNavigationSimulationWidgetInfo ReplyFocusRecipient;
	EUINavigationRule ReplyBoundaryRule;
	FSlateNavigationEventSimulator::ERoutedReason RoutedReason;
	FNavigationSimulationWidgetInfo WidgetThatShouldReceivedFocus;
	FNavigationSimulationWidgetInfo FocusedWidget;
	uint8 bIsDynamic : 1;
	uint8 bAlwaysHandleNavigationAttempt : 1;
	uint8 bCanFindWidgetForSetFocus : 1;
	uint8 bRoutedHandlerHasNavigationMeta : 1;
	uint8 bHandledByViewport : 1;

	void ResetLiveWidget();
};

struct FNavigationSimulationWidgetNode
{
	FNavigationSimulationWidgetNode() = default;
	explicit FNavigationSimulationWidgetNode(ENavigationSimulationNodeType NodeType, const FWidgetPath& InNavigationSource);

	FNavigationSimulationWidgetInfo NavigationSource;
	TArray<FNavigationSimulationWidgetNodeItem, TInlineAllocator<4>> Simulations;
	ENavigationSimulationNodeType NodeType;

	/** Save this node data as a JSON object */
	static TSharedRef<FJsonValue> ToJson(const FNavigationSimulationWidgetNode& Node);

	/** Populate this node data from a JSON object */
	static FNavigationSimulationWidgetNodePtr FromJson(const TSharedRef<FJsonValue>& RootJsonValue);
};


struct FNavigationSimulationNodeUtils
{
	static TArray<FNavigationSimulationWidgetNodePtr> BuildNavigationSimulationNodeListForLive(const TArray<FSlateNavigationEventSimulator::FSimulationResult>& SimulationResult);
	static TArray<FNavigationSimulationWidgetNodePtr> BuildNavigationSimulationNodeListForSnapshot(const TArray<FSlateNavigationEventSimulator::FSimulationResult>& SimulationResult);

	static TArray<FNavigationSimulationWidgetNodePtr> FindReflectorNodes(const TArray<FNavigationSimulationWidgetNodePtr>& Nodes, const TArray<TSharedRef<FWidgetReflectorNodeBase>>& ToFind);

	static int32 IndexOfLiveWidget(const TArray<FNavigationSimulationWidgetNodePtr>& Nodes, const TSharedPtr<const SWidget>& WidgetToFind);
	static int32 IndexOfSnapshotWidget(const TArray<FNavigationSimulationWidgetNodePtr>& Nodes, FNavigationSimulationWidgetInfo::TPointerAsInt WidgetToFind);
};