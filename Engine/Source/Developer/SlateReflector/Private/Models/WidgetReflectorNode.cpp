// Copyright Epic Games, Inc. All Rights Reserved.

#include "Models/WidgetReflectorNode.h"

#include "Algo/Reverse.h"
#include "Modules/ModuleManager.h"
#include "Serialization/JsonTypes.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Layout/Visibility.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/WidgetPath.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Types/ReflectionMetadata.h"
#include "Types/SlateAttributeMetaData.h"
#include "FastUpdate/SlateInvalidationRoot.h"

#define LOCTEXT_NAMESPACE "WidgetReflectorNode"

/**
 * -----------------------------------------------------------------------------
 * FWidgetReflectorNodeBase
 * -----------------------------------------------------------------------------
 */
FWidgetReflectorNodeBase::FWidgetReflectorNodeBase()
	: Tint(FLinearColor::White)
{
	HitTestInfo.IsHitTestVisible = false;
	HitTestInfo.AreChildrenHitTestVisible = false;
}

FWidgetReflectorNodeBase::FWidgetReflectorNodeBase(const FArrangedWidget& InArrangedWidget)
	: WidgetGeometry(InArrangedWidget.Geometry)
	, Tint(FLinearColor::White)
{
	const EVisibility WidgetVisibility = InArrangedWidget.Widget->GetVisibility();
	HitTestInfo.IsHitTestVisible = WidgetVisibility.IsHitTestVisible();
	HitTestInfo.AreChildrenHitTestVisible = WidgetVisibility.AreChildrenHitTestVisible();
}

const FGeometry& FWidgetReflectorNodeBase::GetGeometry() const
{
	return WidgetGeometry;
}

FSlateLayoutTransform FWidgetReflectorNodeBase::GetAccumulatedLayoutTransform() const
{
	return WidgetGeometry.GetAccumulatedLayoutTransform();
}

const FSlateRenderTransform& FWidgetReflectorNodeBase::GetAccumulatedRenderTransform() const
{
	return WidgetGeometry.GetAccumulatedRenderTransform();
}

FVector2f FWidgetReflectorNodeBase::GetLocalSize() const
{
	return WidgetGeometry.GetLocalSize();
}

const FWidgetHitTestInfo& FWidgetReflectorNodeBase::GetHitTestInfo() const
{
	return HitTestInfo;
}

const FLinearColor& FWidgetReflectorNodeBase::GetTint() const
{
	return Tint;
}

void FWidgetReflectorNodeBase::SetTint(const FLinearColor& InTint)
{
	Tint = InTint;
}

void FWidgetReflectorNodeBase::AddChildNode(TSharedRef<FWidgetReflectorNodeBase> InParentNode, TSharedRef<FWidgetReflectorNodeBase> InChildNode)
{
	InParentNode->ChildNodes.Add(MoveTemp(InChildNode));
	InChildNode->ParentNode = InParentNode;
}

const TArray<TSharedRef<FWidgetReflectorNodeBase>>& FWidgetReflectorNodeBase::GetChildNodes() const
{
	return ChildNodes;
}

const TSharedPtr<FWidgetReflectorNodeBase> FWidgetReflectorNodeBase::GetParentNode() const
{
	 return ParentNode.Pin();
}

/**
 * -----------------------------------------------------------------------------
 * FLiveWidgetReflectorNode
 * -----------------------------------------------------------------------------
 */
TSharedRef<FLiveWidgetReflectorNode> FLiveWidgetReflectorNode::Create(const FArrangedWidget& InArrangedWidget)
{
	return MakeShareable(new FLiveWidgetReflectorNode(InArrangedWidget));
}

FLiveWidgetReflectorNode::FLiveWidgetReflectorNode(const FArrangedWidget& InArrangedWidget)
	: FWidgetReflectorNodeBase(InArrangedWidget)
	, Widget(InArrangedWidget.Widget)
{
}

EWidgetReflectorNodeType FLiveWidgetReflectorNode::GetNodeType() const
{
	return EWidgetReflectorNodeType::Live;
}

TSharedPtr<SWidget> FLiveWidgetReflectorNode::GetLiveWidget() const
{
	return Widget.Pin();
}

FText FLiveWidgetReflectorNode::GetWidgetType() const
{
	return FWidgetReflectorNodeUtils::GetWidgetType(Widget.Pin());
}

FText FLiveWidgetReflectorNode::GetWidgetTypeAndShortName() const
{
	return FWidgetReflectorNodeUtils::GetWidgetTypeAndShortName(Widget.Pin());
}

FText FLiveWidgetReflectorNode::GetWidgetVisibilityText() const
{
	return FWidgetReflectorNodeUtils::GetWidgetVisibilityText(Widget.Pin());
}

bool FLiveWidgetReflectorNode::GetWidgetVisible() const
{
	return FWidgetReflectorNodeUtils::GetWidgetVisibility(Widget.Pin());
}

bool FLiveWidgetReflectorNode::GetWidgetVisibilityInherited() const
{
	return FWidgetReflectorNodeUtils::GetWidgetVisibilityInherited(Widget.Pin());
}

bool FLiveWidgetReflectorNode::GetWidgetSupportsInvalidation() const
{
	return FWidgetReflectorNodeUtils::GetWidgetSupportsInvalidation(Widget.Pin());
}

bool FLiveWidgetReflectorNode::GetSupportsInvalidationRecursive() const
{
	return FWidgetReflectorNodeUtils::GetWidgetSupportsInvalidationRecursive(Widget.Pin());
}

FText FLiveWidgetReflectorNode::GetWidgetClippingText() const
{
	return FWidgetReflectorNodeUtils::GetWidgetClippingText(Widget.Pin());
}

int32 FLiveWidgetReflectorNode::GetWidgetLayerId() const
{
	return FWidgetReflectorNodeUtils::GetWidgetLayerId(Widget.Pin());
}
int32 FLiveWidgetReflectorNode::GetWidgetLayerIdOut() const
{
	return FWidgetReflectorNodeUtils::GetWidgetLayerIdOut(Widget.Pin());
}

bool FLiveWidgetReflectorNode::GetWidgetFocusable() const
{
	return FWidgetReflectorNodeUtils::GetWidgetFocusable(Widget.Pin());
}

bool FLiveWidgetReflectorNode::GetWidgetNeedsTick() const
{
	return FWidgetReflectorNodeUtils::GetWidgetNeedsTick(Widget.Pin());
}

bool FLiveWidgetReflectorNode::GetWidgetIsVolatile() const
{
	return FWidgetReflectorNodeUtils::GetWidgetIsVolatile(Widget.Pin());
}

bool FLiveWidgetReflectorNode::GetWidgetIsVolatileIndirectly() const
{
	return FWidgetReflectorNodeUtils::GetWidgetIsVolatileIndirectly(Widget.Pin());
}

bool FLiveWidgetReflectorNode::GetWidgetHasActiveTimers() const
{
	return FWidgetReflectorNodeUtils::GetWidgetHasActiveTimers(Widget.Pin());
}

bool FLiveWidgetReflectorNode::GetWidgetIsInvalidationRoot() const
{
	return FWidgetReflectorNodeUtils::GetWidgetIsInvalidationRoot(Widget.Pin());
}

int32 FLiveWidgetReflectorNode::GetWidgetAttributeCount() const
{
	return FWidgetReflectorNodeUtils::GetWidgetAttributeCount(Widget.Pin());
}

int32 FLiveWidgetReflectorNode::GetWidgetCollapsedAttributeCount() const
{
	return FWidgetReflectorNodeUtils::GetWidgetCollapsedAttributeCount(Widget.Pin());
}

FText FLiveWidgetReflectorNode::GetWidgetReadableLocation() const
{
	return FWidgetReflectorNodeUtils::GetWidgetReadableLocation(Widget.Pin());
}

FString FLiveWidgetReflectorNode::GetWidgetFile() const
{
	return FWidgetReflectorNodeUtils::GetWidgetFile(Widget.Pin());
}

int32 FLiveWidgetReflectorNode::GetWidgetLineNumber() const
{
	return FWidgetReflectorNodeUtils::GetWidgetLineNumber(Widget.Pin());
}

bool FLiveWidgetReflectorNode::HasValidWidgetAssetData() const
{
	return FWidgetReflectorNodeUtils::HasValidWidgetAssetData(Widget.Pin());
}

FAssetData FLiveWidgetReflectorNode::GetWidgetAssetData() const
{
	return FWidgetReflectorNodeUtils::GetWidgetAssetData(Widget.Pin());
}

FVector2D FLiveWidgetReflectorNode::GetWidgetDesiredSize() const
{
	return FWidgetReflectorNodeUtils::GetWidgetDesiredSize(Widget.Pin());
}

FSlateColor FLiveWidgetReflectorNode::GetWidgetForegroundColor() const
{
	return FWidgetReflectorNodeUtils::GetWidgetForegroundColor(Widget.Pin());
}

FWidgetReflectorNodeBase::TPointerAsInt FLiveWidgetReflectorNode::GetWidgetAddress() const
{
	return FWidgetReflectorNodeUtils::GetWidgetAddress(Widget.Pin());
}

bool FLiveWidgetReflectorNode::GetWidgetEnabled() const
{
	return FWidgetReflectorNodeUtils::GetWidgetEnabled(Widget.Pin());
}

UObject* FLiveWidgetReflectorNode::GetWidgetSourceObject() const
{
	if (Widget.IsValid())
	{
		return FReflectionMetaData::GetWidgetSourceObject(*Widget.Pin());
	}

	return nullptr;
}


/**
 * -----------------------------------------------------------------------------
 * FSnapshotWidgetReflectorNode
 * -----------------------------------------------------------------------------
 */
TSharedRef<FSnapshotWidgetReflectorNode> FSnapshotWidgetReflectorNode::Create()
{
	return MakeShareable(new FSnapshotWidgetReflectorNode());
}

TSharedRef<FSnapshotWidgetReflectorNode> FSnapshotWidgetReflectorNode::Create(const FArrangedWidget& InWidgetGeometry)
{
	return MakeShareable(new FSnapshotWidgetReflectorNode(InWidgetGeometry));
}

FSnapshotWidgetReflectorNode::FSnapshotWidgetReflectorNode()
	: bCachedWidgetEnabled(false)
	, CachedWidgetLineNumber(0)
	, CachedWidgetAttributeCount(0)
	, CachedWidgetCollapsedAttributeCount(0)
{
}

FSnapshotWidgetReflectorNode::FSnapshotWidgetReflectorNode(const FArrangedWidget& InArrangedWidget)
	: FWidgetReflectorNodeBase(InArrangedWidget)
	, CachedWidgetType(FWidgetReflectorNodeUtils::GetWidgetType(InArrangedWidget.Widget))
	, CachedWidgetTypeAndShortName(FWidgetReflectorNodeUtils::GetWidgetTypeAndShortName(InArrangedWidget.Widget))
	, CachedWidgetVisibilityText(FWidgetReflectorNodeUtils::GetWidgetVisibilityText(InArrangedWidget.Widget))
	, bCachedWidgetVisible(FWidgetReflectorNodeUtils::GetWidgetVisibility(InArrangedWidget.Widget))
	, bCachedWidgetVisibleInherited(FWidgetReflectorNodeUtils::GetWidgetVisibilityInherited(InArrangedWidget.Widget))
	, bCachedSupportsInvalidation(FWidgetReflectorNodeUtils::GetWidgetSupportsInvalidation(InArrangedWidget.Widget))
	, bCachedSupportsInvalidationRecursive(FWidgetReflectorNodeUtils::GetWidgetSupportsInvalidationRecursive(InArrangedWidget.Widget))
	, bCachedWidgetFocusable(FWidgetReflectorNodeUtils::GetWidgetFocusable(InArrangedWidget.Widget))
	, bCachedWidgetNeedsTick(FWidgetReflectorNodeUtils::GetWidgetNeedsTick(InArrangedWidget.Widget))
	, bCachedWidgetIsVolatile(FWidgetReflectorNodeUtils::GetWidgetIsVolatile(InArrangedWidget.Widget))
	, bCachedWidgetIsVolatileIndirectly(FWidgetReflectorNodeUtils::GetWidgetIsVolatileIndirectly(InArrangedWidget.Widget))
	, bCachedWidgetHasActiveTimers(FWidgetReflectorNodeUtils::GetWidgetHasActiveTimers(InArrangedWidget.Widget))
	, bCachedWidgetIsInvalidationRoot(FWidgetReflectorNodeUtils::GetWidgetIsInvalidationRoot(InArrangedWidget.Widget))
	, bCachedWidgetEnabled(FWidgetReflectorNodeUtils::GetWidgetEnabled(InArrangedWidget.Widget))
	, CachedWidgetClippingText(FWidgetReflectorNodeUtils::GetWidgetClippingText(InArrangedWidget.Widget))
	, CachedWidgetLayerId(FWidgetReflectorNodeUtils::GetWidgetLayerId(InArrangedWidget.Widget))
	, CachedWidgetLayerIdOut(FWidgetReflectorNodeUtils::GetWidgetLayerIdOut(InArrangedWidget.Widget))
	, CachedWidgetReadableLocation(FWidgetReflectorNodeUtils::GetWidgetReadableLocation(InArrangedWidget.Widget))
	, CachedWidgetFile(FWidgetReflectorNodeUtils::GetWidgetFile(InArrangedWidget.Widget))
	, CachedWidgetLineNumber(FWidgetReflectorNodeUtils::GetWidgetLineNumber(InArrangedWidget.Widget))
	, CachedWidgetAttributeCount(FWidgetReflectorNodeUtils::GetWidgetAttributeCount(InArrangedWidget.Widget))
	, CachedWidgetCollapsedAttributeCount(FWidgetReflectorNodeUtils::GetWidgetCollapsedAttributeCount(InArrangedWidget.Widget))
	, CachedWidgetAssetData(FWidgetReflectorNodeUtils::GetWidgetAssetData(InArrangedWidget.Widget))
	, CachedWidgetDesiredSize(FWidgetReflectorNodeUtils::GetWidgetDesiredSize(InArrangedWidget.Widget))
	, CachedWidgetForegroundColor(FWidgetReflectorNodeUtils::GetWidgetForegroundColor(InArrangedWidget.Widget))
	, CachedWidgetAddress(FWidgetReflectorNodeUtils::GetWidgetAddress(InArrangedWidget.Widget))
{
}

EWidgetReflectorNodeType FSnapshotWidgetReflectorNode::GetNodeType() const
{
	return EWidgetReflectorNodeType::Snapshot;
}

TSharedPtr<SWidget> FSnapshotWidgetReflectorNode::GetLiveWidget() const
{
	return nullptr;
}

FText FSnapshotWidgetReflectorNode::GetWidgetType() const
{
	return CachedWidgetType;
}

FText FSnapshotWidgetReflectorNode::GetWidgetTypeAndShortName() const
{
	return CachedWidgetTypeAndShortName;
}

FText FSnapshotWidgetReflectorNode::GetWidgetVisibilityText() const
{
	return CachedWidgetVisibilityText;
}

bool FSnapshotWidgetReflectorNode::GetWidgetFocusable() const
{
	return bCachedWidgetFocusable;
}

bool FSnapshotWidgetReflectorNode::GetWidgetVisible() const
{
	return bCachedWidgetVisible;
}

bool FSnapshotWidgetReflectorNode::GetWidgetVisibilityInherited() const
{
	return bCachedWidgetVisibleInherited;
}

bool FSnapshotWidgetReflectorNode::GetWidgetSupportsInvalidation() const
{
	return bCachedSupportsInvalidation;
}

bool FSnapshotWidgetReflectorNode::GetSupportsInvalidationRecursive() const
{
	return bCachedSupportsInvalidationRecursive;
}

FText FSnapshotWidgetReflectorNode::GetWidgetClippingText() const
{
	return CachedWidgetClippingText;
}

int32 FSnapshotWidgetReflectorNode::GetWidgetLayerId() const
{
	return CachedWidgetLayerId;
}

int32 FSnapshotWidgetReflectorNode::GetWidgetLayerIdOut() const
{
	return CachedWidgetLayerIdOut;
}

bool FSnapshotWidgetReflectorNode::GetWidgetNeedsTick() const
{
	return bCachedWidgetNeedsTick;
}

bool FSnapshotWidgetReflectorNode::GetWidgetIsVolatile() const
{
	return bCachedWidgetIsVolatile;
}

bool FSnapshotWidgetReflectorNode::GetWidgetIsVolatileIndirectly() const
{
	return bCachedWidgetIsVolatileIndirectly;
}

bool FSnapshotWidgetReflectorNode::GetWidgetHasActiveTimers() const
{
	return bCachedWidgetHasActiveTimers;
}

bool FSnapshotWidgetReflectorNode::GetWidgetIsInvalidationRoot() const
{
	return bCachedWidgetIsInvalidationRoot;
}

FText FSnapshotWidgetReflectorNode::GetWidgetReadableLocation() const
{
	return CachedWidgetReadableLocation;
}

FString FSnapshotWidgetReflectorNode::GetWidgetFile() const
{
	return CachedWidgetFile;
}

int32 FSnapshotWidgetReflectorNode::GetWidgetLineNumber() const
{
	return CachedWidgetLineNumber;
}

int32 FSnapshotWidgetReflectorNode::GetWidgetAttributeCount() const
{
	return CachedWidgetAttributeCount;
}

int32 FSnapshotWidgetReflectorNode::GetWidgetCollapsedAttributeCount() const
{
	return CachedWidgetCollapsedAttributeCount;
}

bool FSnapshotWidgetReflectorNode::HasValidWidgetAssetData() const
{
	return CachedWidgetAssetData.IsValid();
}

FAssetData FSnapshotWidgetReflectorNode::GetWidgetAssetData() const
{
	return CachedWidgetAssetData;
}

FVector2D FSnapshotWidgetReflectorNode::GetWidgetDesiredSize() const
{
	return CachedWidgetDesiredSize;
}

FSlateColor FSnapshotWidgetReflectorNode::GetWidgetForegroundColor() const
{
	return CachedWidgetForegroundColor;
}

FWidgetReflectorNodeBase::TPointerAsInt FSnapshotWidgetReflectorNode::GetWidgetAddress() const
{
	return CachedWidgetAddress;
}

bool FSnapshotWidgetReflectorNode::GetWidgetEnabled() const
{
	return bCachedWidgetEnabled;
}

UObject* FSnapshotWidgetReflectorNode::GetWidgetSourceObject() const
{
	return nullptr;
}

TSharedRef<FJsonValue> FSnapshotWidgetReflectorNode::ToJson(const TSharedRef<FSnapshotWidgetReflectorNode>& RootSnapshotNode)
{
	struct Internal
	{
		static TSharedRef<FJsonValue> CreateVector2DJsonValue(const FVector2D& InVec2D)
		{
			TArray<TSharedPtr<FJsonValue>> StructJsonArray;
			StructJsonArray.Add(MakeShareable(new FJsonValueNumber(InVec2D.X)));
			StructJsonArray.Add(MakeShareable(new FJsonValueNumber(InVec2D.Y)));
			return MakeShareable(new FJsonValueArray(StructJsonArray));
		}
		static TSharedRef<FJsonValue> CreateVector2DJsonValue(const FVector2f& InVec2D)
		{
			TArray<TSharedPtr<FJsonValue>> StructJsonArray;
			StructJsonArray.Add(MakeShareable(new FJsonValueNumber(InVec2D.X)));
			StructJsonArray.Add(MakeShareable(new FJsonValueNumber(InVec2D.Y)));
			return MakeShareable(new FJsonValueArray(StructJsonArray));
		}
		static TSharedRef<FJsonValue> CreateMatrix2x2JsonValue(const FMatrix2x2& InMatrix)
		{
			float m00, m01, m10, m11;
			InMatrix.GetMatrix(m00, m01, m10, m11);

			TArray<TSharedPtr<FJsonValue>> StructJsonArray;
			StructJsonArray.Add(MakeShareable(new FJsonValueNumber(m00)));
			StructJsonArray.Add(MakeShareable(new FJsonValueNumber(m01)));
			StructJsonArray.Add(MakeShareable(new FJsonValueNumber(m10)));
			StructJsonArray.Add(MakeShareable(new FJsonValueNumber(m11)));
			return MakeShareable(new FJsonValueArray(StructJsonArray));
		}

		static TSharedRef<FJsonValue> CreateSlateLayoutTransformJsonValue(const FSlateLayoutTransform& InLayoutTransform)
		{
			TSharedRef<FJsonObject> StructJsonObject = MakeShareable(new FJsonObject());
			StructJsonObject->SetNumberField(TEXT("Scale"), InLayoutTransform.GetScale());
			StructJsonObject->SetField(TEXT("Translation"), CreateVector2DJsonValue(InLayoutTransform.GetTranslation()));
			return MakeShareable(new FJsonValueObject(StructJsonObject));
		}

		static TSharedRef<FJsonValue> CreateSlateRenderTransformJsonValue(const FSlateRenderTransform& InRenderTransform)
		{
			TSharedRef<FJsonObject> StructJsonObject = MakeShareable(new FJsonObject());
			StructJsonObject->SetField(TEXT("Matrix"), CreateMatrix2x2JsonValue(InRenderTransform.GetMatrix()));
			StructJsonObject->SetField(TEXT("Translation"), CreateVector2DJsonValue(InRenderTransform.GetTranslation()));
			return MakeShareable(new FJsonValueObject(StructJsonObject));
		}

		static TSharedRef<FJsonValue> CreateLinearColorJsonValue(const FLinearColor& InColor)
		{
			TArray<TSharedPtr<FJsonValue>> StructJsonArray;
			StructJsonArray.Add(MakeShareable(new FJsonValueNumber(InColor.R)));
			StructJsonArray.Add(MakeShareable(new FJsonValueNumber(InColor.G)));
			StructJsonArray.Add(MakeShareable(new FJsonValueNumber(InColor.B)));
			StructJsonArray.Add(MakeShareable(new FJsonValueNumber(InColor.A)));
			return MakeShareable(new FJsonValueArray(StructJsonArray));
		}

		static TSharedRef<FJsonValue> CreateSlateColorJsonValue(const FSlateColor& InColor)
		{
			const bool bIsColorSpecified = InColor.IsColorSpecified();
			const FLinearColor ColorToUse = (bIsColorSpecified) ? InColor.GetSpecifiedColor() : FLinearColor::White;

			TSharedRef<FJsonObject> StructJsonObject = MakeShareable(new FJsonObject());
			StructJsonObject->SetBoolField(TEXT("IsColorSpecified"), bIsColorSpecified);
			StructJsonObject->SetField(TEXT("Color"), CreateLinearColorJsonValue(ColorToUse));
			return MakeShareable(new FJsonValueObject(StructJsonObject));
		}

		static TSharedRef<FJsonValue> CreateWidgetHitTestInfoJsonValue(const FWidgetHitTestInfo& InHitTestInfo)
		{
			TSharedRef<FJsonObject> StructJsonObject = MakeShareable(new FJsonObject());
			StructJsonObject->SetBoolField(TEXT("IsHitTestVisible"), InHitTestInfo.IsHitTestVisible);
			StructJsonObject->SetBoolField(TEXT("AreChildrenHitTestVisible"), InHitTestInfo.AreChildrenHitTestVisible);
			return MakeShareable(new FJsonValueObject(StructJsonObject));
		}

		static FString ConvertPtrIntToString(FWidgetReflectorNodeBase::TPointerAsInt Value)
		{
			return FWidgetReflectorNodeUtils::WidgetAddressToString(Value);
		}
	};

	TSharedRef<FJsonObject> RootJsonObject = MakeShareable(new FJsonObject());

	/**
	 *  Do not forget to change the version number (SWidgetSnapshotVisualizer.cpp) if you change something here
	 */

	RootJsonObject->SetField(TEXT("AccumulatedLayoutTransform"), Internal::CreateSlateLayoutTransformJsonValue(RootSnapshotNode->GetAccumulatedLayoutTransform()));
	RootJsonObject->SetField(TEXT("AccumulatedRenderTransform"), Internal::CreateSlateRenderTransformJsonValue(RootSnapshotNode->GetAccumulatedRenderTransform()));
	RootJsonObject->SetField(TEXT("LocalSize"), Internal::CreateVector2DJsonValue(RootSnapshotNode->GetLocalSize()));
	RootJsonObject->SetField(TEXT("HitTestInfo"), Internal::CreateWidgetHitTestInfoJsonValue(RootSnapshotNode->HitTestInfo));
	RootJsonObject->SetField(TEXT("Tint"), Internal::CreateLinearColorJsonValue(RootSnapshotNode->Tint));
	RootJsonObject->SetStringField(TEXT("WidgetType"), RootSnapshotNode->CachedWidgetType.ToString());
	RootJsonObject->SetStringField(TEXT("WidgetTypeAndShortName"), RootSnapshotNode->CachedWidgetTypeAndShortName.ToString());
	RootJsonObject->SetStringField(TEXT("WidgetVisibilityText"), RootSnapshotNode->CachedWidgetVisibilityText.ToString());
	RootJsonObject->SetBoolField(TEXT("WidgetVisible"), RootSnapshotNode->bCachedWidgetVisible);
	RootJsonObject->SetBoolField(TEXT("WidgetVisibleInherited"), RootSnapshotNode->bCachedWidgetVisibleInherited);
	RootJsonObject->SetBoolField(TEXT("SupportsInvalidation"), RootSnapshotNode->bCachedSupportsInvalidation);
	RootJsonObject->SetBoolField(TEXT("SupportsInvalidationRecursive"), RootSnapshotNode->bCachedSupportsInvalidationRecursive);
	RootJsonObject->SetBoolField(TEXT("WidgetFocusable"), RootSnapshotNode->bCachedWidgetFocusable);
	RootJsonObject->SetBoolField(TEXT("WidgetNeedsTick"), RootSnapshotNode->bCachedWidgetNeedsTick);
	RootJsonObject->SetBoolField(TEXT("WidgetIsVolatile"), RootSnapshotNode->bCachedWidgetIsVolatile);
	RootJsonObject->SetBoolField(TEXT("WidgetIsVolatileIndirectly"), RootSnapshotNode->bCachedWidgetIsVolatileIndirectly);
	RootJsonObject->SetBoolField(TEXT("WidgetHasActiveTimers"), RootSnapshotNode->bCachedWidgetHasActiveTimers);
	RootJsonObject->SetBoolField(TEXT("WidgetIsInvalidationRoot"), RootSnapshotNode->bCachedWidgetIsInvalidationRoot);
	RootJsonObject->SetBoolField(TEXT("WidgetEnabled"), RootSnapshotNode->bCachedWidgetEnabled);
	RootJsonObject->SetStringField(TEXT("WidgetClippingText"), RootSnapshotNode->CachedWidgetClippingText.ToString());
	RootJsonObject->SetNumberField(TEXT("WidgetLayerId"), RootSnapshotNode->CachedWidgetLayerId);
	RootJsonObject->SetNumberField(TEXT("WidgetLayerIdOut"), RootSnapshotNode->CachedWidgetLayerIdOut);
	RootJsonObject->SetStringField(TEXT("WidgetReadableLocation"), RootSnapshotNode->CachedWidgetReadableLocation.ToString());
	RootJsonObject->SetStringField(TEXT("WidgetFile"), RootSnapshotNode->CachedWidgetFile);
	RootJsonObject->SetNumberField(TEXT("WidgetLineNumber"), RootSnapshotNode->CachedWidgetLineNumber);
	RootJsonObject->SetNumberField(TEXT("WidgetAttributeCount"), RootSnapshotNode->CachedWidgetAttributeCount);
	RootJsonObject->SetNumberField(TEXT("WidgetCollapsedAttributeCount"), RootSnapshotNode->CachedWidgetCollapsedAttributeCount);
	RootJsonObject->SetField(TEXT("WidgetDesiredSize"), Internal::CreateVector2DJsonValue(RootSnapshotNode->CachedWidgetDesiredSize));
	RootJsonObject->SetField(TEXT("WidgetForegroundColor"), Internal::CreateSlateColorJsonValue(RootSnapshotNode->CachedWidgetForegroundColor));
	RootJsonObject->SetStringField(TEXT("WidgetAddress"), Internal::ConvertPtrIntToString(RootSnapshotNode->CachedWidgetAddress));
	RootJsonObject->SetStringField(TEXT("WidgetAssetPath"), RootSnapshotNode->CachedWidgetAssetData.GetObjectPathString());

	TArray<TSharedPtr<FJsonValue>> ChildNodesJsonArray;
	for (const auto& ChildReflectorNode : RootSnapshotNode->ChildNodes)
	{
		check(ChildReflectorNode->GetNodeType() == EWidgetReflectorNodeType::Snapshot);
		ChildNodesJsonArray.Add(FSnapshotWidgetReflectorNode::ToJson(StaticCastSharedRef<FSnapshotWidgetReflectorNode>(ChildReflectorNode)));
	}
	RootJsonObject->SetArrayField(TEXT("ChildNodes"), ChildNodesJsonArray);

	return MakeShareable(new FJsonValueObject(RootJsonObject));
}

TSharedRef<FSnapshotWidgetReflectorNode> FSnapshotWidgetReflectorNode::FromJson(const TSharedRef<FJsonValue>& RootJsonValue)
{
	struct Internal
	{
		static FVector2D ParseVector2DJsonValue(const TSharedPtr<FJsonValue>& InJsonValue)
		{
			if (!InJsonValue.IsValid())
			{
				return FVector2D::ZeroVector;
			}
			const TArray<TSharedPtr<FJsonValue>>& StructJsonArray = InJsonValue->AsArray();
			check(StructJsonArray.Num() == 2);

			return FVector2D(
				(float)StructJsonArray[0]->AsNumber(),
				(float)StructJsonArray[1]->AsNumber()
				);
		}

		static FMatrix2x2 ParseMatrix2x2JsonValue(const TSharedPtr<FJsonValue>& InJsonValue)
		{
			if (!InJsonValue.IsValid())
			{
				return FMatrix2x2();
			}
			const TArray<TSharedPtr<FJsonValue>>& StructJsonArray = InJsonValue->AsArray();
			check(StructJsonArray.Num() == 4);

			return FMatrix2x2(
				(float)StructJsonArray[0]->AsNumber(), 
				(float)StructJsonArray[1]->AsNumber(), 
				(float)StructJsonArray[2]->AsNumber(),
				(float)StructJsonArray[3]->AsNumber()
				);
		}

		static FSlateLayoutTransform ParseSlateLayoutTransformJsonValue(const TSharedPtr<FJsonValue>& InJsonValue)
		{
			if (!InJsonValue.IsValid())
			{
				return FSlateLayoutTransform();
			}
			const TSharedPtr<FJsonObject>& StructJsonObject = InJsonValue->AsObject();
			check(StructJsonObject.IsValid());

			return FSlateLayoutTransform(
				(float)StructJsonObject->GetNumberField(TEXT("Scale")),
				ParseVector2DJsonValue(StructJsonObject->GetField<EJson::None>(TEXT("Translation")))
				);
		}

		static FSlateRenderTransform ParseSlateRenderTransformJsonValue(const TSharedPtr<FJsonValue>& InJsonValue)
		{
			if (!InJsonValue.IsValid())
			{
				return FSlateRenderTransform();
			}
			const TSharedPtr<FJsonObject>& StructJsonObject = InJsonValue->AsObject();
			check(StructJsonObject.IsValid());

			return FSlateRenderTransform(
				ParseMatrix2x2JsonValue(StructJsonObject->GetField<EJson::None>(TEXT("Matrix"))),
				ParseVector2DJsonValue(StructJsonObject->GetField<EJson::None>(TEXT("Translation")))
				);
		}

		static FLinearColor ParseLinearColorJsonValue(const TSharedPtr<FJsonValue>& InJsonValue)
		{
			if(!InJsonValue.IsValid())
			{
				return FLinearColor(EForceInit::ForceInit);
			}
			const TArray<TSharedPtr<FJsonValue>>& StructJsonArray = InJsonValue->AsArray();
			check(StructJsonArray.Num() == 4);

			return FLinearColor(
				(float)StructJsonArray[0]->AsNumber(),
				(float)StructJsonArray[1]->AsNumber(),
				(float)StructJsonArray[2]->AsNumber(),
				(float)StructJsonArray[3]->AsNumber()
				);
		}

		static FSlateColor ParseSlateColorJsonValue(const TSharedPtr<FJsonValue>& InJsonValue)
		{
			if (!InJsonValue.IsValid())
			{
				return FSlateColor();
			}
			const TSharedPtr<FJsonObject>& StructJsonObject = InJsonValue->AsObject();
			check(StructJsonObject.IsValid());

			const bool bIsColorSpecified = StructJsonObject->GetBoolField(TEXT("IsColorSpecified"));
			if (bIsColorSpecified)
			{
				return FSlateColor(ParseLinearColorJsonValue(StructJsonObject->GetField<EJson::None>(TEXT("Color"))));
			}
			else
			{
				return FSlateColor::UseForeground();
			}
		}

		static FWidgetHitTestInfo ParseWidgetHitTestInfoJsonValue(const TSharedPtr<FJsonValue>& InJsonValue)
		{
			if (!InJsonValue.IsValid())
			{
				return FWidgetHitTestInfo();
			}
			const TSharedPtr<FJsonObject>& StructJsonObject = InJsonValue->AsObject();
			check(StructJsonObject.IsValid());

			FWidgetHitTestInfo HitTestInfo;
			HitTestInfo.IsHitTestVisible = StructJsonObject->GetBoolField(TEXT("IsHitTestVisible"));
			HitTestInfo.AreChildrenHitTestVisible = StructJsonObject->GetBoolField(TEXT("AreChildrenHitTestVisible"));
			return HitTestInfo;
		}

		static FWidgetReflectorNodeBase::TPointerAsInt ParsePtrIntFromString(const FString& Value)
		{
			FWidgetReflectorNodeBase::TPointerAsInt Result = 0;
			LexFromString(Result, *Value);
			return Result;
		}
	};

	const TSharedPtr<FJsonObject>& RootJsonObject = RootJsonValue->AsObject();
	check(RootJsonObject.IsValid());

	auto RootSnapshotNode = FSnapshotWidgetReflectorNode::Create();

	const FSlateLayoutTransform LayoutTransform = Internal::ParseSlateLayoutTransformJsonValue(RootJsonObject->GetField<EJson::None>(TEXT("AccumulatedLayoutTransform")));
	const FSlateRenderTransform RenderTransform = Internal::ParseSlateRenderTransformJsonValue(RootJsonObject->GetField<EJson::None>(TEXT("AccumulatedRenderTransform")));
	const FVector2D LocalSize = Internal::ParseVector2DJsonValue(RootJsonObject->GetField<EJson::None>(TEXT("LocalSize")));
	RootSnapshotNode->WidgetGeometry = FGeometry::MakeRoot(LocalSize, LayoutTransform, RenderTransform);
	RootSnapshotNode->HitTestInfo = Internal::ParseWidgetHitTestInfoJsonValue(RootJsonObject->GetField<EJson::None>(TEXT("HitTestInfo")));
	RootSnapshotNode->Tint = Internal::ParseLinearColorJsonValue(RootJsonObject->GetField<EJson::None>(TEXT("Tint")));

	RootSnapshotNode->CachedWidgetType = FText::FromString(RootJsonObject->GetStringField(TEXT("WidgetType")));
	RootSnapshotNode->CachedWidgetTypeAndShortName = FText::FromString(RootJsonObject->GetStringField(TEXT("WidgetTypeAndShortName")));
	RootSnapshotNode->CachedWidgetVisibilityText = FText::FromString(RootJsonObject->GetStringField(TEXT("WidgetVisibilityText")));
	RootSnapshotNode->bCachedWidgetVisible = RootJsonObject->GetBoolField(TEXT("WidgetVisible"));
	RootSnapshotNode->bCachedWidgetVisibleInherited = RootJsonObject->GetBoolField(TEXT("WidgetVisibleInherited"));
	RootSnapshotNode->bCachedSupportsInvalidation = RootJsonObject->GetBoolField(TEXT("SupportsInvalidation"));
	RootSnapshotNode->bCachedSupportsInvalidationRecursive = RootJsonObject->GetBoolField(TEXT("SupportsInvalidationRecursive"));
	RootSnapshotNode->bCachedWidgetFocusable = RootJsonObject->GetBoolField(TEXT("WidgetFocusable"));
	RootSnapshotNode->bCachedWidgetNeedsTick = RootJsonObject->GetBoolField(TEXT("WidgetNeedsTick"));
	RootSnapshotNode->bCachedWidgetIsVolatile = RootJsonObject->GetBoolField(TEXT("WidgetIsVolatile"));
	RootSnapshotNode->bCachedWidgetIsVolatileIndirectly = RootJsonObject->GetBoolField(TEXT("WidgetIsVolatileIndirectly"));
	RootSnapshotNode->bCachedWidgetHasActiveTimers = RootJsonObject->GetBoolField(TEXT("WidgetHasActiveTimers"));
	RootSnapshotNode->bCachedWidgetIsInvalidationRoot = RootJsonObject->GetBoolField(TEXT("WidgetIsInvalidationRoot"));
	RootSnapshotNode->bCachedWidgetEnabled = RootJsonObject->GetBoolField(TEXT("WidgetEnabled"));
	RootSnapshotNode->CachedWidgetClippingText = FText::FromString(RootJsonObject->GetStringField(TEXT("WidgetClippingText")));
	RootSnapshotNode->CachedWidgetLayerId = RootJsonObject->GetIntegerField(TEXT("WidgetLayerId"));
	RootSnapshotNode->CachedWidgetLayerIdOut = RootJsonObject->GetIntegerField(TEXT("WidgetLayerIdOut"));
	RootSnapshotNode->CachedWidgetReadableLocation = FText::FromString(RootJsonObject->GetStringField(TEXT("WidgetReadableLocation")));
	RootSnapshotNode->CachedWidgetFile = RootJsonObject->GetStringField(TEXT("WidgetFile"));
	RootSnapshotNode->CachedWidgetLineNumber = RootJsonObject->GetIntegerField(TEXT("WidgetLineNumber"));
	RootSnapshotNode->CachedWidgetAttributeCount = RootJsonObject->GetIntegerField(TEXT("WidgetAttributeCount"));
	RootSnapshotNode->CachedWidgetCollapsedAttributeCount = RootJsonObject->GetIntegerField(TEXT("WidgetCollapsedAttributeCount"));
	RootSnapshotNode->CachedWidgetDesiredSize = Internal::ParseVector2DJsonValue(RootJsonObject->GetField<EJson::None>(TEXT("WidgetDesiredSize")));
	RootSnapshotNode->CachedWidgetForegroundColor = Internal::ParseSlateColorJsonValue(RootJsonObject->GetField<EJson::None>(TEXT("WidgetForegroundColor")));
	RootSnapshotNode->CachedWidgetAddress = Internal::ParsePtrIntFromString(RootJsonObject->GetStringField(TEXT("WidgetAddress")));

	FSoftObjectPath AssetPath(RootJsonObject->GetStringField(TEXT("WidgetAssetPath")));
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	RootSnapshotNode->CachedWidgetAssetData = AssetRegistry.GetAssetByObjectPath(AssetPath);

	const TArray<TSharedPtr<FJsonValue>>& ChildNodesJsonArray = RootJsonObject->GetArrayField(TEXT("ChildNodes"));
	for (const TSharedPtr<FJsonValue>& ChildNodeJsonValue : ChildNodesJsonArray)
	{
		FSnapshotWidgetReflectorNode::AddChildNode(RootSnapshotNode, FSnapshotWidgetReflectorNode::FromJson(ChildNodeJsonValue.ToSharedRef()));
	}

	return RootSnapshotNode;
}

/**
 * -----------------------------------------------------------------------------
 * FWidgetReflectorNodeUtils
 * -----------------------------------------------------------------------------
 */
TSharedRef<FLiveWidgetReflectorNode> FWidgetReflectorNodeUtils::NewLiveNode(const FArrangedWidget& InWidgetGeometry)
{
	return StaticCastSharedRef<FLiveWidgetReflectorNode>(NewNode(EWidgetReflectorNodeType::Live, InWidgetGeometry));
}

TSharedRef<FLiveWidgetReflectorNode> FWidgetReflectorNodeUtils::NewLiveNodeTreeFrom(const FArrangedWidget& InWidgetGeometry)
{
	return StaticCastSharedRef<FLiveWidgetReflectorNode>(NewNodeTreeFrom(EWidgetReflectorNodeType::Live, InWidgetGeometry));
}

TSharedRef<FSnapshotWidgetReflectorNode> FWidgetReflectorNodeUtils::NewSnapshotNode(const FArrangedWidget& InWidgetGeometry)
{
	return StaticCastSharedRef<FSnapshotWidgetReflectorNode>(NewNode(EWidgetReflectorNodeType::Snapshot, InWidgetGeometry));
}

TSharedRef<FSnapshotWidgetReflectorNode> FWidgetReflectorNodeUtils::NewSnapshotNodeTreeFrom(const FArrangedWidget& InWidgetGeometry)
{
	return StaticCastSharedRef<FSnapshotWidgetReflectorNode>(NewNodeTreeFrom(EWidgetReflectorNodeType::Snapshot, InWidgetGeometry));
}

TSharedRef<FWidgetReflectorNodeBase> FWidgetReflectorNodeUtils::NewNode(const EWidgetReflectorNodeType InNodeType, const FArrangedWidget& InWidgetGeometry)
{
	switch (InNodeType)
	{
	case EWidgetReflectorNodeType::Live:
		return FLiveWidgetReflectorNode::Create(InWidgetGeometry);
	case EWidgetReflectorNodeType::Snapshot:
		return FSnapshotWidgetReflectorNode::Create(InWidgetGeometry);
	default:
		// Should never reach this point, but we have to return something!
		check(false);
		return FLiveWidgetReflectorNode::Create(InWidgetGeometry);
	}
}

TSharedRef<FWidgetReflectorNodeBase> FWidgetReflectorNodeUtils::NewNodeTreeFrom(const EWidgetReflectorNodeType InNodeType, const FArrangedWidget& InWidgetGeometry)
{
	TSharedRef<FWidgetReflectorNodeBase> NewNodeInstance = NewNode(InNodeType, InWidgetGeometry);

	TSharedRef<SWidget> CurWidgetParent = InWidgetGeometry.Widget;
#if WITH_SLATE_DEBUGGING
	FChildren* Children = CurWidgetParent->Debug_GetChildrenForReflector();
#else
	FChildren* Children = CurWidgetParent->GetChildren();
#endif

	auto BuildChild = [NewNodeInstance, CurWidgetParent, InNodeType](const TSharedRef<SWidget>& ChildWidget)
	{
		FGeometry ChildGeometry = ChildWidget->GetCachedGeometry();
		const EVisibility CurWidgetVisibility = ChildWidget->GetVisibility();

		// Don't add geometry for completely collapsed stuff
		if (CurWidgetVisibility == EVisibility::Collapsed)
		{
			ChildGeometry = FGeometry();
		}
		else if (!CurWidgetParent->ValidatePathToChild(&ChildWidget.Get()))
		{
			ChildGeometry = FGeometry();
		}

		// Note that we include both visible and invisible children!
		FSnapshotWidgetReflectorNode::AddChildNode(NewNodeInstance, NewNodeTreeFrom(InNodeType, FArrangedWidget(ChildWidget, ChildGeometry)));
	};

	if (ensure(Children))
	{
		for (int32 ChildIndex = 0; ChildIndex < Children->Num(); ++ChildIndex)
		{
			TSharedRef<SWidget> ChildWidget = Children->GetChildAt(ChildIndex);
			BuildChild(ChildWidget);
		}
	}

	return NewNodeInstance;
}

namespace WidgetReflectorNodeUtilsImpl
{
	void FindLiveWidgetPath(const TArray<TSharedRef<FWidgetReflectorNodeBase>>& TreeNodes, const FWidgetPath& WidgetPathToFind, TArray<TSharedRef<FWidgetReflectorNodeBase>>& SearchResult, int32 PathIndexToStart)
	{
		for (int32 PathIndex = PathIndexToStart; PathIndex < WidgetPathToFind.Widgets.Num(); ++PathIndex)
		{
			const FArrangedWidget& WidgetToFind = WidgetPathToFind.Widgets[PathIndex];
			const FWidgetReflectorNodeBase::TPointerAsInt WidgetPathToFindAddress = ::FWidgetReflectorNodeUtils::GetWidgetAddress(WidgetToFind.Widget);

			for (int32 NodeIndex = 0; NodeIndex < TreeNodes.Num(); ++NodeIndex)
			{
				if (TreeNodes[NodeIndex]->GetWidgetAddress() == WidgetPathToFindAddress)
				{
					SearchResult.Add(TreeNodes[NodeIndex]);
					FindLiveWidgetPath(TreeNodes[NodeIndex]->GetChildNodes(), WidgetPathToFind, SearchResult, PathIndex + 1);
				}
			}
		}
	}

	void FindLiveWidget(const TSharedPtr<const SWidget>& InWidgetToFind, const TSharedRef<FWidgetReflectorNodeBase>& InNodeToTest, TArray<TSharedRef<FWidgetReflectorNodeBase>>& FoundReversedList)
	{
		if (InNodeToTest->GetLiveWidget() == InWidgetToFind)
		{
			FoundReversedList.Add(InNodeToTest);
		}
		else
		{
			for (const TSharedRef<FWidgetReflectorNodeBase>& Child : InNodeToTest->GetChildNodes())
			{
				FindLiveWidget(InWidgetToFind, Child, FoundReversedList);
				if (FoundReversedList.Num() > 0)
				{
					FoundReversedList.Add(InNodeToTest);
					break;
				}
			}
		}
	}

	void FindSnapshotWidget(FWidgetReflectorNodeBase::TPointerAsInt InWidgetToFind, const TSharedRef<FWidgetReflectorNodeBase>& InNodeToTest, TArray<TSharedRef<FWidgetReflectorNodeBase>>& FoundReversedList)
	{
		if (InNodeToTest->GetWidgetAddress() == InWidgetToFind)
		{
			FoundReversedList.Add(InNodeToTest);
		}
		else
		{
			for (const TSharedRef<FWidgetReflectorNodeBase>& Child : InNodeToTest->GetChildNodes())
			{
				FindSnapshotWidget(InWidgetToFind, Child, FoundReversedList);
				if (FoundReversedList.Num() > 0)
				{
					FoundReversedList.Add(InNodeToTest);
					break;
				}
			}
		}
	}
}

void FWidgetReflectorNodeUtils::FindLiveWidgetPath(const TArray<TSharedRef<FWidgetReflectorNodeBase>>& TreeNodes, const FWidgetPath& WidgetPathToFind, TArray<TSharedRef<FWidgetReflectorNodeBase>>& SearchResult)
{
	SearchResult.Reset();
	if (WidgetPathToFind.Widgets.Num() == 0)
	{
		return;
	}
	WidgetReflectorNodeUtilsImpl::FindLiveWidgetPath(TreeNodes, WidgetPathToFind, SearchResult, 0);
}

void FWidgetReflectorNodeUtils::FindLiveWidget(const TArray<TSharedRef<FWidgetReflectorNodeBase>>& CandidateNodes, const TSharedPtr<const SWidget>& WidgetToFind, TArray<TSharedRef<FWidgetReflectorNodeBase>>& SearchResult)
{
	SearchResult.Reset();
	for (const TSharedRef<FWidgetReflectorNodeBase>& Itt : CandidateNodes)
	{
		WidgetReflectorNodeUtilsImpl::FindLiveWidget(WidgetToFind, Itt, SearchResult);
		if (SearchResult.Num() > 0)
		{
			Algo::Reverse(SearchResult);
			break;
		}
	}
}

void FWidgetReflectorNodeUtils::FindSnaphotWidget(const TArray<TSharedRef<FWidgetReflectorNodeBase>>& CandidateNodes, FWidgetReflectorNodeBase::TPointerAsInt WidgetToFind, TArray<TSharedRef<FWidgetReflectorNodeBase>>& SearchResult)
{
	SearchResult.Reset();
	for (const TSharedRef<FWidgetReflectorNodeBase>& Itt : CandidateNodes)
	{
		WidgetReflectorNodeUtilsImpl::FindSnapshotWidget(WidgetToFind, Itt, SearchResult);
		if (SearchResult.Num() > 0)
		{
			Algo::Reverse(SearchResult);
			break;
		}
	}
}

FText FWidgetReflectorNodeUtils::GetWidgetType(const TSharedPtr<const SWidget>& InWidget)
{
	return (InWidget.IsValid()) ? FText::FromString(InWidget->GetTypeAsString()) : FText::GetEmpty();
}

FText FWidgetReflectorNodeUtils::GetWidgetTypeAndShortName(const TSharedPtr<const SWidget>& InWidget)
{
	if (InWidget.IsValid())
	{
		FText WidgetType = GetWidgetType(InWidget);

		// UMG widgets have meta-data to help track them
		TSharedPtr<FReflectionMetaData> MetaData = InWidget->GetMetaData<FReflectionMetaData>();
		if (MetaData.IsValid())
		{
			if (MetaData->Name != NAME_None)
			{
				return FText::Format(LOCTEXT("WidgetTypeAndName", "{0} ({1})"), WidgetType, FText::FromName(MetaData->Name));
			}
		}

		return WidgetType;
	}

	return FText::GetEmpty();
}

FText FWidgetReflectorNodeUtils::GetWidgetVisibilityText(const TSharedPtr<const SWidget>& InWidget)
{
	return (InWidget.IsValid()) ? FText::FromString(InWidget->GetVisibility().ToString()) : FText::GetEmpty();
}

bool FWidgetReflectorNodeUtils::GetWidgetVisibility(const TSharedPtr<const SWidget>& InWidget)
{
	return InWidget.IsValid() ? InWidget->GetVisibility().IsVisible() : false;
}

bool FWidgetReflectorNodeUtils::GetWidgetVisibilityInherited(const TSharedPtr<const SWidget>& InWidget)
{
	return InWidget.IsValid() ? InWidget->GetProxyHandle().GetWidgetVisibility(InWidget.Get()).IsVisible() : false;
}

bool FWidgetReflectorNodeUtils::GetWidgetSupportsInvalidation(const TSharedPtr<const SWidget>& InWidget)
{
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
	return InWidget.IsValid() ? InWidget->SupportsInvalidation() : false;
#else
	return false;
#endif
}

bool FWidgetReflectorNodeUtils::GetWidgetSupportsInvalidationRecursive(const TSharedPtr<const SWidget>& InWidget)
{
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
	return InWidget.IsValid() ? InWidget->SupportsInvalidationRecursive(SWidget::EInvalidationStrategy::UseCachedValue) : false;
#else
	return false;
#endif
}

bool FWidgetReflectorNodeUtils::GetWidgetFocusable(const TSharedPtr<const SWidget>& InWidget)
{
	return InWidget.IsValid() ? InWidget->SupportsKeyboardFocus() : false;
}

bool FWidgetReflectorNodeUtils::GetWidgetNeedsTick(const TSharedPtr<const SWidget>& InWidget)
{
	return InWidget.IsValid() ? InWidget->GetCanTick() : false;
}

bool FWidgetReflectorNodeUtils::GetWidgetIsVolatile(const TSharedPtr<const SWidget>& InWidget)
{
	return InWidget.IsValid() ? InWidget->IsVolatile() : false;
}

bool FWidgetReflectorNodeUtils::GetWidgetIsVolatileIndirectly(const TSharedPtr<const SWidget>& InWidget)
{
	return InWidget.IsValid() ? InWidget->IsVolatileIndirectly() : false;
}

bool FWidgetReflectorNodeUtils::GetWidgetHasActiveTimers(const TSharedPtr<const SWidget>& InWidget)
{
	return InWidget.IsValid() ? InWidget->HasActiveTimers() : false;
}

bool FWidgetReflectorNodeUtils::GetWidgetIsInvalidationRoot(const TSharedPtr<const SWidget>& InWidget)
{
	return InWidget.IsValid() ? InWidget->Advanced_IsInvalidationRoot() : false;
}

int32 FWidgetReflectorNodeUtils::GetWidgetAttributeCount(const TSharedPtr<const SWidget>& InWidget)
{
	if (InWidget.IsValid())
	{
		if (FSlateAttributeMetaData* MetaData = FSlateAttributeMetaData::FindMetaData(*InWidget.Get()))
		{
			return MetaData->GetRegisteredAttributeCount();
		}
	}
	return 0;
}

int32 FWidgetReflectorNodeUtils::GetWidgetCollapsedAttributeCount(const TSharedPtr<const SWidget>& InWidget)
{
	if (InWidget.IsValid())
	{
		if (FSlateAttributeMetaData* MetaData = FSlateAttributeMetaData::FindMetaData(*InWidget.Get()))
		{
			return MetaData->GetRegisteredAffectVisibilityAttributeCount();
		}
	}
	return 0;
}

FText FWidgetReflectorNodeUtils::GetWidgetClippingText(const TSharedPtr<const SWidget>& InWidget)
{
	if ( InWidget.IsValid() )
	{
		switch ( InWidget->GetClipping() )
		{
		case EWidgetClipping::Inherit:
			return LOCTEXT("WidgetClippingNo", "No");
		case EWidgetClipping::ClipToBounds:
			return LOCTEXT("WidgetClippingYes", "Yes");
		case EWidgetClipping::ClipToBoundsAlways:
			return LOCTEXT("WidgetClippingYesAlways", "Yes (Always)");
		case EWidgetClipping::ClipToBoundsWithoutIntersecting:
			return LOCTEXT("WidgetClippingYesWithoutIntersecting", "Yes (No Intersect)");
		case EWidgetClipping::OnDemand:
			return LOCTEXT("WidgetClippingOnDemand", "On Demand");
		}
	}

	return FText::GetEmpty();
}

int32 FWidgetReflectorNodeUtils::GetWidgetLayerId(const TSharedPtr<const SWidget>& InWidget)
{
	return (InWidget.IsValid()) ? InWidget->GetPersistentState().LayerId : -1;
}

int32 FWidgetReflectorNodeUtils::GetWidgetLayerIdOut(const TSharedPtr<const SWidget>& InWidget)
{
	return (InWidget.IsValid()) ? InWidget->GetPersistentState().OutgoingLayerId : -1;
}

FText FWidgetReflectorNodeUtils::GetWidgetReadableLocation(const TSharedPtr<const SWidget>& InWidget)
{
	return FText::FromString(FReflectionMetaData::GetWidgetDebugInfo(InWidget.Get()));
}

FString FWidgetReflectorNodeUtils::GetWidgetFile(const TSharedPtr<const SWidget>& InWidget)
{
	return (InWidget.IsValid()) ? InWidget->GetCreatedInLocation().GetPlainNameString() : FString();
}

int32 FWidgetReflectorNodeUtils::GetWidgetLineNumber(const TSharedPtr<const SWidget>& InWidget)
{
	return (InWidget.IsValid()) ? InWidget->GetCreatedInLocation().GetNumber() : 0;
}

bool FWidgetReflectorNodeUtils::HasValidWidgetAssetData(const TSharedPtr<const SWidget>& InWidget)
{
	if (InWidget.IsValid())
	{
		if (TSharedPtr<FReflectionMetaData> MetaData = InWidget->GetMetaData<FReflectionMetaData>())
		{
			return MetaData->Asset.IsValid();
		}
	}

	return false;
}

FAssetData FWidgetReflectorNodeUtils::GetWidgetAssetData(const TSharedPtr<const SWidget>& InWidget)
{
	if (InWidget.IsValid())
	{
		// UMG widgets have meta-data to help track them
		TSharedPtr<FReflectionMetaData> MetaData = InWidget->GetMetaData<FReflectionMetaData>();
		if (MetaData.IsValid() && MetaData->Asset.Get() != nullptr)
		{
			return FAssetData(MetaData->Asset.Get());
		}
	}

	return FAssetData();
}

FVector2D FWidgetReflectorNodeUtils::GetWidgetDesiredSize(const TSharedPtr<const SWidget>& InWidget)
{
	return (InWidget.IsValid()) ? InWidget->GetDesiredSize() : FVector2D::ZeroVector;
}

FWidgetReflectorNodeBase::TPointerAsInt FWidgetReflectorNodeUtils::GetWidgetAddress(const TSharedPtr<const SWidget>& InWidget)
{
	return static_cast<FWidgetReflectorNodeBase::TPointerAsInt>(reinterpret_cast<PTRINT>(InWidget.Get()));
}

FString FWidgetReflectorNodeUtils::WidgetAddressToString(FWidgetReflectorNodeBase::TPointerAsInt InWidgetPtr)
{
	return FString::Printf(TEXT("0x%0llx"), InWidgetPtr);
}

FSlateColor FWidgetReflectorNodeUtils::GetWidgetForegroundColor(const TSharedPtr<const SWidget>& InWidget)
{
	return (InWidget.IsValid()) ? InWidget->GetForegroundColor() : FSlateColor::UseForeground();
}

bool FWidgetReflectorNodeUtils::GetWidgetEnabled(const TSharedPtr<const SWidget>& InWidget)
{
	return (InWidget.IsValid()) ? InWidget->IsEnabled() : false;
}

#undef LOCTEXT_NAMESPACE
