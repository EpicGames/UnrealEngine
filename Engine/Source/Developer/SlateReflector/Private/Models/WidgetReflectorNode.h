// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Layout/Geometry.h"
#include "Widgets/SNullWidget.h"
#include "Layout/ArrangedWidget.h"
#include "AssetRegistry/AssetData.h"

class FJsonValue;
class FWidgetPath;
class SWidget;
class UObject;

/**
 * Enum used as crude RTTI for the widget reflector node types
 */
enum class EWidgetReflectorNodeType : uint8
{
	Live,
	Snapshot,
};

/**
 * Cached information about whether a widget can be hit-tested
 */
struct FWidgetHitTestInfo
{
	bool IsHitTestVisible = false;
	bool AreChildrenHitTestVisible = false;
};

/** 
 * A widget reflector node that contains the interface and basic data required by both live and snapshot nodes 
 */
class FWidgetReflectorNodeBase
{
public:
	using TPointerAsInt = uint64; // We can't use PTRINT since the target may be 32bits and the json could be saved in 64bits

	/**
	 * Destructor
	 */
	virtual ~FWidgetReflectorNodeBase() = default;

	/**
	 * @return Get the enum entry corresponding to this type of widget reflector node (used as crude RTTI)
	 */
	virtual EWidgetReflectorNodeType GetNodeType() const = 0;

	/**
	 * @note This function only works for ULiveWidgetReflectorNode instances
	 * @return The live widget that this node is referencing
	 */
	virtual TSharedPtr<SWidget> GetLiveWidget() const = 0;

	/**
	 * @return The type string for the widget we were initialized from
	 */
	virtual FText GetWidgetType() const = 0;

	/**
	 * @return The type string for the widget we were initialized from
	 */
	virtual FText GetWidgetTypeAndShortName() const = 0;
	
	/**
	 * @return The visibility string for the widget we were initialized from
	 */
	virtual FText GetWidgetVisibilityText() const = 0;

	/**
	 * @return The clipping string for the widget we were initialized from
	 */
	virtual FText GetWidgetClippingText() const = 0;

	/**
	 * @return The LayerId the widget were painted from
	 */
	virtual int32 GetWidgetLayerId() const = 0;
	virtual int32 GetWidgetLayerIdOut() const = 0;

	/**
	 * @return The bool indicating whether or not the widget we were initialized from reports as Focusable
	 */
	virtual bool GetWidgetFocusable() const = 0;

	/**
	 * @return The bool indicating whether or not the widget we were initialized from reports is visible
	 */
	virtual bool GetWidgetVisible() const = 0;

	/**
	 * @return The bool indicating whether or not the widget we were initialized from reports is visible
	 */
	virtual bool GetWidgetVisibilityInherited() const = 0;

	/**
	 * @return The bool indicating whether or not the widget we were initialized from reports as supporting invalidation
	 */
	virtual bool GetWidgetSupportsInvalidation() const = 0;

	/**
	 * @return The bool indicating whether or not the widget we were initialized from reports as recursively supporting invalidation
	 */
	virtual bool GetSupportsInvalidationRecursive() const = 0;

	/**
	 * @return The bool indicating whether or not the widget we were initialized from reports needs tick
	 */
	virtual bool GetWidgetNeedsTick() const = 0;

	/**
	 * @return The bool indicating whether or not the widget we were initialized from reports is volatile
	 */
	virtual bool GetWidgetIsVolatile() const = 0;

	/**
	 * @return The bool indicating whether or not the widget we were initialized from reports is volatile indirectly
	 */
	virtual bool GetWidgetIsVolatileIndirectly() const = 0;

	/**
	 * @return The bool indicating whether or not the widget we were initialized from reports has active timers
	 */
	virtual bool GetWidgetHasActiveTimers() const = 0;

	/**
	 * @return The bool indicating whether or not the widget we were initialized from reports is an Invalidation Root
	 */
	virtual bool GetWidgetIsInvalidationRoot() const = 0;

	/**
	 * @return the numbers of registered slate attributes on the widget we were initialized from
	 */
	virtual int32 GetWidgetAttributeCount() const = 0;

	/**
	 * @return the numbers of registered slate attributes marked as "update when collapsed" on the widget we were initialized from
	 */
	virtual int32 GetWidgetCollapsedAttributeCount() const = 0;

	/**
	 * The human readable location for widgets that are defined in C++ is the file and line number
	 * The human readable location for widgets that are defined in UMG is the asset name
	 * @return The fully human readable location for the widget we were initialized from
	 */
	virtual FText GetWidgetReadableLocation() const = 0;

	/**
	 * @return The name of the file that the widget we were initialized from was created from (for C++ widgets)
	 */
	virtual FString GetWidgetFile() const = 0;

	/**
	 * @return The line number of the file that the widget we were initialized from was created from (for C++ widgets)
	 */
	virtual int32 GetWidgetLineNumber() const = 0;

	/**
	 * @return true if the data of the asset that the widget we were initialized from was created from (for UMG widgets) is valid
	 */
	virtual bool HasValidWidgetAssetData() const = 0;

	/**
	 * @return The data of the asset that the widget we were initialized from was created from (for UMG widgets)
	 */
	virtual FAssetData GetWidgetAssetData() const = 0;

	/**
	 * @return The desired size of the widget we were initialized from
	 */
	virtual FVector2D GetWidgetDesiredSize() const = 0;

	/**
	 * @return The foreground color of the widget we were initialized from
	 */
	virtual FSlateColor GetWidgetForegroundColor() const = 0;

	/**
	 * @return The in-memory address of the widget we were initialized from
	 */
	virtual TPointerAsInt GetWidgetAddress() const = 0;

	/**
	 * @return True if the the widget we were initialized from is enabled, false otherwise
	 */
	virtual bool GetWidgetEnabled() const = 0;

	/**
	 * @return True if the widget is live, and local and can potentially be manipulated in real-time.
	 */
	virtual bool IsWidgetLive() const = 0;

	/**
	 * @return The source object obtained from the widget's meta data.
	 */
	virtual UObject* GetWidgetSourceObject() const = 0;

	/**
	 * @return The geometry of the widget.
	 */
	const FGeometry& GetGeometry() const;

	/**
	 * @return The accumulated layout transform of the widget we were initialized from
	 */
	FSlateLayoutTransform GetAccumulatedLayoutTransform() const;

	/**
	 * @return The accumulated render transform of the widget we were initialized from
	 */
	const FSlateRenderTransform& GetAccumulatedRenderTransform() const;

	/**
	 * @return The local size of the widget we were initialized from
	 */
	FVector2f GetLocalSize() const;

	/**
	 * @return The basic hit-test of the widget we were initialized from
	 */
	const FWidgetHitTestInfo& GetHitTestInfo() const;

	/**
	 * @return The tint that is applied to text in order to provide visual hints
	 */
	const FLinearColor& GetTint() const;

	/**
	 * Set the tint to the given value
	 */
	void SetTint(const FLinearColor& InTint);

	/**
	 * Add the given node to our list of children for this widget (this node will keep a strong reference to the instance)
	 */
	static void AddChildNode(TSharedRef<FWidgetReflectorNodeBase> InParentNode, TSharedRef<FWidgetReflectorNodeBase> InChildNode);

	/**
	 * @return The node entries for the widget's children
	 */
	const TArray<TSharedRef<FWidgetReflectorNodeBase>>& GetChildNodes() const;

	/**
	 * @returns The node entry for the widget's parent, if it exists, 
	 */
	const TSharedPtr<FWidgetReflectorNodeBase> GetParentNode() const;

protected:
	/**
	 * Default constructor
	 */
	FWidgetReflectorNodeBase();

	/**
	 * Construct this node from the given widget geometry, caching out any data that may be required for future visualization in the widget reflector
	 */
	explicit FWidgetReflectorNodeBase(const FArrangedWidget& InWidgetGeometry);

protected:
	/** The Geometry of the widget. */
	FGeometry WidgetGeometry;

	/** The hit-test information for the widget */
	FWidgetHitTestInfo HitTestInfo;

	/** Node entries for the widget's children */
	TArray<TSharedRef<FWidgetReflectorNodeBase>> ChildNodes;

	/** Node entry for the widget's parent  */
	TWeakPtr<FWidgetReflectorNodeBase> ParentNode;

	/** A tint that is applied to text in order to provide visual hints (Transient) */
	FLinearColor Tint;
};

/** 
 * A widget reflector node that holds on to the widget it references so that certain properties can be updated live 
 */
class FLiveWidgetReflectorNode : public FWidgetReflectorNodeBase
{
public:
	/**
	 * Destructor
	 */
	virtual ~FLiveWidgetReflectorNode() = default;

	/**
	 * Create a live node instance from the given widget geometry, caching out any data that may be required for future visualization in the widget reflector
	 */
	static TSharedRef<FLiveWidgetReflectorNode> Create(const FArrangedWidget& InWidgetGeometry);

	// FWidgetReflectorNodeBase interface
	virtual EWidgetReflectorNodeType GetNodeType() const override;
	virtual TSharedPtr<SWidget> GetLiveWidget() const override;
	virtual FText GetWidgetType() const override;
	virtual FText GetWidgetTypeAndShortName() const override;
	virtual FText GetWidgetVisibilityText() const override;
	virtual bool GetWidgetVisible() const override;
	virtual bool GetWidgetVisibilityInherited() const override;
	virtual bool GetWidgetSupportsInvalidation() const override;
	virtual bool GetSupportsInvalidationRecursive() const override;
	virtual FText GetWidgetClippingText() const override;
	virtual int32 GetWidgetLayerId() const override;
	virtual int32 GetWidgetLayerIdOut() const override;
	virtual bool GetWidgetFocusable() const override;
	virtual bool GetWidgetNeedsTick() const override;
	virtual bool GetWidgetIsVolatile() const override;
	virtual bool GetWidgetIsVolatileIndirectly() const override;
	virtual bool GetWidgetHasActiveTimers() const override;
	virtual bool GetWidgetIsInvalidationRoot() const override;
	virtual FText GetWidgetReadableLocation() const override;
	virtual FString GetWidgetFile() const override;
	virtual int32 GetWidgetLineNumber() const override;
	virtual int32 GetWidgetAttributeCount() const override;
	virtual int32 GetWidgetCollapsedAttributeCount() const override;
	virtual bool HasValidWidgetAssetData() const override;
	virtual FAssetData GetWidgetAssetData() const override;
	virtual FVector2D GetWidgetDesiredSize() const override;
	virtual FSlateColor GetWidgetForegroundColor() const override;
	virtual TPointerAsInt GetWidgetAddress() const override;
	virtual bool GetWidgetEnabled() const override;
	virtual bool IsWidgetLive() const override { return true; }
	virtual UObject* GetWidgetSourceObject() const override;

private:
	/**
	 * Construct this node from the given widget geometry, caching out any data that may be required for future visualization in the widget reflector
	 */
	explicit FLiveWidgetReflectorNode(const FArrangedWidget& InWidgetGeometry);

private:
	/** The widget this node is watching */
	TWeakPtr<SWidget> Widget;
};

/** 
 * A widget reflector node that holds the widget information from a snapshot at a given point in time 
 */
class FSnapshotWidgetReflectorNode : public FWidgetReflectorNodeBase
{
public:
	/**
	 * Destructor
	 */
	virtual ~FSnapshotWidgetReflectorNode() = default;

	/**
	 * Create a default snapshot node instance
	 */
	static TSharedRef<FSnapshotWidgetReflectorNode> Create();

	/**
	 * Create a snapshot node instance node from the given widget geometry, caching out any data that may be required for future visualization in the widget reflector
	 */
	static TSharedRef<FSnapshotWidgetReflectorNode> Create(const FArrangedWidget& InWidgetGeometry);

	// FWidgetReflectorNodeBase interface
	virtual EWidgetReflectorNodeType GetNodeType() const override;
	virtual TSharedPtr<SWidget> GetLiveWidget() const override;
	virtual FText GetWidgetType() const override;
	virtual FText GetWidgetTypeAndShortName() const override;
	virtual FText GetWidgetVisibilityText() const override;
	virtual FText GetWidgetClippingText() const override;
	virtual int32 GetWidgetLayerId() const override;
	virtual int32 GetWidgetLayerIdOut() const override;
	virtual bool GetWidgetVisible() const override;
	virtual bool GetWidgetVisibilityInherited() const override;
	virtual bool GetWidgetSupportsInvalidation() const override;
	virtual bool GetSupportsInvalidationRecursive() const override;
	virtual bool GetWidgetFocusable() const override;
	virtual bool GetWidgetNeedsTick() const override;
	virtual bool GetWidgetIsVolatile() const override;
	virtual bool GetWidgetIsVolatileIndirectly() const override;
	virtual bool GetWidgetHasActiveTimers() const override;
	virtual bool GetWidgetIsInvalidationRoot() const override;
	virtual FText GetWidgetReadableLocation() const override;
	virtual FString GetWidgetFile() const override;
	virtual int32 GetWidgetLineNumber() const override;
	virtual int32 GetWidgetAttributeCount() const override;
	virtual int32 GetWidgetCollapsedAttributeCount() const override;
	virtual bool HasValidWidgetAssetData() const override;
	virtual FAssetData GetWidgetAssetData() const override;
	virtual FVector2D GetWidgetDesiredSize() const override;
	virtual FSlateColor GetWidgetForegroundColor() const override;
	virtual TPointerAsInt GetWidgetAddress() const override;
	virtual bool GetWidgetEnabled() const override;
	virtual bool IsWidgetLive() const override { return false; }
	virtual UObject* GetWidgetSourceObject() const override;

	/** Save this node data as a JSON object */
	static TSharedRef<FJsonValue> ToJson(const TSharedRef<FSnapshotWidgetReflectorNode>& RootSnapshotNode);

	/** Populate this node data from a JSON object */
	static TSharedRef<FSnapshotWidgetReflectorNode> FromJson(const TSharedRef<FJsonValue>& RootJsonValue);

private:
	/**
	 * Default constructor
	 */
	FSnapshotWidgetReflectorNode();

	/**
	 * Construct this node from the given widget geometry, caching out any data that may be required for future visualization in the widget reflector
	 */
	explicit FSnapshotWidgetReflectorNode(const FArrangedWidget& InWidgetGeometry);

private:
	/** The type string of the widget at the point it was passed to Initialize */
	FText CachedWidgetType;

	/** The type and short name string of the widget at the point it was passed to Initialize */
	FText CachedWidgetTypeAndShortName;

	/** The visibility string of the widget at the point it was passed to Initialize */
	FText CachedWidgetVisibilityText;

	/** The visible of the widget at the point it was passed to Initialize */
	bool bCachedWidgetVisible;
	
	/** The visible inherited of the widget at the point it was passed to Initialize */
	bool bCachedWidgetVisibleInherited;
	
	/** Did the widget support invalidation at the point it was passed to Initialize */
	bool bCachedSupportsInvalidation;
	
	/** Did the widget recursively support invalidation at the point it was passed to Initialize */
	bool bCachedSupportsInvalidationRecursive;

	/** The focusability of the widget at the point it was passed to Initialize */
	bool bCachedWidgetFocusable;

	/** The ticking state of the widget at the point it was passed to Initialize */
	bool bCachedWidgetNeedsTick;

	/** The volatility state of the widget at the point it was passed to Initialize */
	bool bCachedWidgetIsVolatile;

	/** The volatility indirectly state of the widget at the point it was passed to Initialize */
	bool bCachedWidgetIsVolatileIndirectly;

	/** The active timer state of the widget at the point it was passed to Initialize */
	bool bCachedWidgetHasActiveTimers;
	
	/** If the widget was an invalidation root at the point it was passed to Initialize */
	bool bCachedWidgetIsInvalidationRoot;

	/** The enabled state of the widget at the point it was passed to Initialize */
	bool bCachedWidgetEnabled;
	
	/** The clipping string of the widget at the point it was passed to Initialize */
	FText CachedWidgetClippingText;

	/** The LayerId of the widget */
	int32 CachedWidgetLayerId;
	int32 CachedWidgetLayerIdOut;

	/** The human readable location (source file for C++ widgets, asset name for UMG widgets) of the widget at the point it was passed to Initialize */
	FText CachedWidgetReadableLocation;

	/** The name of the file that the widget was created from at the point it was passed to Initialize (for C++ widgets) */
	FString CachedWidgetFile;

	/** The line number of the file that the widget was created from at the point it was passed to Initialize (for C++ widgets) */
	int32 CachedWidgetLineNumber;

	/** The number of slate attributes registered on the SWidget. */
	int32 CachedWidgetAttributeCount;
	
	/** The number of slate attributes registered on the SWidget marked as "update when collapsed". */
	int32 CachedWidgetCollapsedAttributeCount;

	/** The name of the asset that the widget was created from at the point it was passed to Initialize (for UMG widgets) */
	FAssetData CachedWidgetAssetData;

	/** The desired size of the widget at the point it was passed to Initialize */
	FVector2D CachedWidgetDesiredSize;

	/** The foreground color of the widget at the point it was passed to Initialize */
	FSlateColor CachedWidgetForegroundColor;

	/** The in-memory address of the widget at the point it was passed to Initialize */
	TPointerAsInt CachedWidgetAddress;
};


class FWidgetReflectorNodeUtils
{
public:
	/**
	 * Create a single node referencing a live widget.
	 *
	 * @param InNodeClass The type of widget reflector node to create
	 * @param InWidgetGeometry Optional widget and associated geometry which this node should represent
	 */
	static TSharedRef<FLiveWidgetReflectorNode> NewLiveNode(const FArrangedWidget& InWidgetGeometry = FArrangedWidget(SNullWidget::NullWidget, FGeometry()));

	/**
	 * Create nodes for the supplied widget and all their children such that they reference a live widget
	 * Note that we include both visible and invisible children!
	 * 
	 * @param InNodeClass The type of widget reflector node to create
	 * @param InWidgetGeometry Widget and geometry whose children to capture in the snapshot.
	 */
	static TSharedRef<FLiveWidgetReflectorNode> NewLiveNodeTreeFrom(const FArrangedWidget& InWidgetGeometry);

	/**
	 * Create a single node referencing a snapshot of its current state.
	 *
	 * @param InNodeClass The type of widget reflector node to create
	 * @param InWidgetGeometry Optional widget and associated geometry which this node should represent
	 */
	static TSharedRef<FSnapshotWidgetReflectorNode> NewSnapshotNode(const FArrangedWidget& InWidgetGeometry = FArrangedWidget(SNullWidget::NullWidget, FGeometry()));

	/**
	 * Create nodes for the supplied widget and all their children such that they reference a snapshot of their current state
	 * Note that we include both visible and invisible children!
	 * 
	 * @param InNodeClass The type of widget reflector node to create
	 * @param InWidgetGeometry Widget and geometry whose children to capture in the snapshot.
	 */
	static TSharedRef<FSnapshotWidgetReflectorNode> NewSnapshotNodeTreeFrom(const FArrangedWidget& InWidgetGeometry);

private:
	/**
	 * Create a single node.
	 *
	 * @param InNodeClass The type of widget reflector node to create
	 * @param InWidgetGeometry Optional widget and associated geometry which this node should represent
	 */
	static TSharedRef<FWidgetReflectorNodeBase> NewNode(const EWidgetReflectorNodeType InNodeType, const FArrangedWidget& InWidgetGeometry = FArrangedWidget(SNullWidget::NullWidget, FGeometry()));

	/**
	 * Create nodes for the supplied widget and all their children
	 * Note that we include both visible and invisible children!
	 * 
	 * @param InNodeClass The type of widget reflector node to create
	 * @param InWidgetGeometry Widget and geometry whose children to capture in the snapshot.
	 */
	static TSharedRef<FWidgetReflectorNodeBase> NewNodeTreeFrom(const EWidgetReflectorNodeType InNodeType, const FArrangedWidget& InWidgetGeometry);

public:
	/**
	 * Locate all the widgets from a widget path in a list of nodes and their children.
	 * @note This only really works for live nodes, as the snapshot nodes may no longer exist, or not even be local to this machine
	 *
	 * @param CandidateNodes A list of FReflectorNodes that represent widgets.
	 * @param WidgetPathToFind We want to find all reflector nodes corresponding to widgets in this path
	 * @param SearchResult An array that gets results put in it
	 */
	static void FindLiveWidgetPath(const TArray<TSharedRef<FWidgetReflectorNodeBase>>& CandidateNodes, const FWidgetPath& WidgetPathToFind, TArray<TSharedRef<FWidgetReflectorNodeBase>>& SearchResult);

	/**
	 * Locate the widget in a list of nodes and their children.
	 * @note This only really works for live nodes, as the snapshot nodes may no longer exist, or not even be local to this machine
	 *
	 * @param CandidateNodes A list of FReflectorNodes that represent widgets.
	 * @param WidgetToFind We want to find the reflector nodes corresponding to this widget
	 * @param SearchResult An array that gets results put in it
	 */
	static void FindLiveWidget(const TArray<TSharedRef<FWidgetReflectorNodeBase>>& CandidateNodes, const TSharedPtr<const SWidget>& WidgetToFind, TArray<TSharedRef<FWidgetReflectorNodeBase>>& SearchResult);

	/**
	 * Locate the widget pointer in a list of nodes and their children.
	 * @note This only really works for snapshot nodes, as the live node may no longer have the same pointer
	 *
	 * @param CandidateNodes A list of FReflectorNodes that represent widgets.
	 * @param WidgetPointer We want to find all reflector nodes corresponding to widgets in this path
	 * @param SearchResult An array that gets results put in it
	 */
	static void FindSnaphotWidget(const TArray<TSharedRef<FWidgetReflectorNodeBase>>& CandidateNodes, FWidgetReflectorNodeBase::TPointerAsInt WidgetToFind, TArray<TSharedRef<FWidgetReflectorNodeBase>>& SearchResult);

public:
	/**
	 * @return The type string for the given widget
	 */
	static FText GetWidgetType(const TSharedPtr<const SWidget>& InWidget);

	/**
	 * @return The type string combined with a short name (if any) for the given widget
	 */
	static FText GetWidgetTypeAndShortName(const TSharedPtr<const SWidget>& InWidget);
	
	/**
	 * @return The current visibility string for the given widget
	 */
	static FText GetWidgetVisibilityText(const TSharedPtr<const SWidget>& InWidget);

	/** Is the widget visible? */
	static bool GetWidgetVisibility(const TSharedPtr<const SWidget>& InWidget);

	/** Is the widget visible and its parents are also visible? */
	static bool GetWidgetVisibilityInherited(const TSharedPtr<const SWidget>& InWidget);
	
	/** Does the widget supports invalidation? */
	static bool GetWidgetSupportsInvalidation(const TSharedPtr<const SWidget>& InWidget);

	/** Does this widget and all its children support invalidation? */
	static bool GetWidgetSupportsInvalidationRecursive(const TSharedPtr<const SWidget>& InWidget);

	/**
	 * @return The current clipping string for the given widget
	 */
	static FText GetWidgetClippingText(const TSharedPtr<const SWidget>& InWidget);

	/**
	 * @return The current LayerId for the given widget
	 */
	static int32 GetWidgetLayerId(const TSharedPtr<const SWidget>& InWidget);
	static int32 GetWidgetLayerIdOut(const TSharedPtr<const SWidget>& InWidget);

	/**
	* @return The current focusability for the given widget
	*/
	static bool GetWidgetFocusable(const TSharedPtr<const SWidget>& InWidget);

	static bool GetWidgetNeedsTick(const TSharedPtr<const SWidget>& InWidget);
	static bool GetWidgetIsVolatile(const TSharedPtr<const SWidget>& InWidget);
	static bool GetWidgetIsVolatileIndirectly(const TSharedPtr<const SWidget>& InWidget);
	static bool GetWidgetHasActiveTimers(const TSharedPtr<const SWidget>& InWidget);
	static bool GetWidgetIsInvalidationRoot(const TSharedPtr<const SWidget>& InWidget);
	static int32 GetWidgetAttributeCount(const TSharedPtr<const SWidget>& InWidget);
	static int32 GetWidgetCollapsedAttributeCount(const TSharedPtr<const SWidget>& InWidget);
	
	/**
	 * The human readable location for widgets that are defined in C++ is the file and line number
	 * The human readable location for widgets that are defined in UMG is the asset name
	 * @return The fully human readable location for the given widget
	 */
	static FText GetWidgetReadableLocation(const TSharedPtr<const SWidget>& InWidget);
	
	/**
	 * @return The name of the file that this widget was created from (for C++ widgets)
	 */
	static FString GetWidgetFile(const TSharedPtr<const SWidget>& InWidget);
	
	/**
	 * @return The line number of the file that this widget was created from (for C++ widgets)
	 */
	static int32 GetWidgetLineNumber(const TSharedPtr<const SWidget>& InWidget);

	/**
	 * @return true if the name of the asset that this widget was created from (for UMG widgets) is valid
	 */
	static bool HasValidWidgetAssetData(const TSharedPtr<const SWidget>& InWidget);

	/**
	 * @return The name of the asset that this widget was created from (for UMG widgets)
	 */
	static FAssetData GetWidgetAssetData(const TSharedPtr<const SWidget>& InWidget);

	/**
	 * @return The current desired size of the given widget
	 */
	static FVector2D GetWidgetDesiredSize(const TSharedPtr<const SWidget>& InWidget);

	/**
	 * @return The in-memory address of the widget, converted to a string
	 */
	static FString GetWidgetAddressAsString(const TSharedPtr<const SWidget>& InWidget);

	/**
	 * @return The in-memory address of the widget
	 */
	static FWidgetReflectorNodeBase::TPointerAsInt GetWidgetAddress(const TSharedPtr<const SWidget>& InWidget);

	/**
	 * @return Convert a widget address into a string
	 */
	static FString WidgetAddressToString(FWidgetReflectorNodeBase::TPointerAsInt InWidgetPtr);

	/**
	 * @return The current foreground color of the given widget
	 */
	static FSlateColor GetWidgetForegroundColor(const TSharedPtr<const SWidget>& InWidget);

	/**
	 * @return True if the given widget is currently enabled, false otherwise
	 */
	static bool GetWidgetEnabled(const TSharedPtr<const SWidget>& InWidget);
};
