// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateNavigationEventSimulator.h"

#include "Application/SlateWindowHelper.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/IConsoleManager.h"
#include "Input/HittestGrid.h"
#include "Input/NavigationReply.h"
#include "Models/WidgetReflectorNode.h"
#include "SlateReflectorModule.h"
#include "Types/NavigationMetaData.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "SlateNavigationSimulatioN"

namespace SlateNavigationSimulationInternal
{
	EUINavigation FourCardinalDirections[] = { EUINavigation::Left, EUINavigation::Up, EUINavigation::Right, EUINavigation::Down };
	EUINavigation NextPrevious[] = { EUINavigation::Next, EUINavigation::Previous };

	static FAutoConsoleCommand CSlateNavigationSimulate(
		TEXT("Slate.Navigation.Simulate"),
		TEXT("Log the result of what the widget may do when it received a navigation event.")
		TEXT("Use: \"Slate.Navigation.Simulate Widget=0x00AABBCCDD Navigation=UINavigationIndex [UserIndex=Number] [Genesis=NavigationGenesisIndex]\"")
		TEXT("UINavigationIndex use: 0 for Left, 1 for Right, 2 for Up, 3 for Down, 4 for Next, 5 for Previous")
		TEXT("NavigationGenesisIndex use: 0 for Keyboard|, 1 for Controller, 2 for User"),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld*, FOutputDevice& Ar)
			{
				int32 UserIndex = 0;
				EUINavigation UINavigation = EUINavigation::Left;
				ENavigationGenesis Genesis = ENavigationGenesis::Controller;
				const SWidget* WidgetPtr = nullptr;
				for(const FString& Arg : Args)
				{
					FString WidgetAsString;
					int64 NavigationIndex = 0;
					int64 GenesisIndex = 0;
					if (FParse::Value(*Arg, TEXT("Widget="), WidgetAsString))
					{
						PTRINT WidgetPtrAsInt = FCString::Strtoui64(*WidgetAsString, nullptr, 0);
						WidgetPtr = reinterpret_cast<const SWidget*>(WidgetPtrAsInt);
					}
					else if (FParse::Value(*Arg, TEXT("Navigation="), NavigationIndex))
					{
						if (NavigationIndex >= 0 && NavigationIndex <= 5)
						{
							UINavigation = static_cast<EUINavigation>(NavigationIndex);
						}
					}
					else if (FParse::Value(*Arg, TEXT("UserIndex="), UserIndex))
					{

					}
					else if (FParse::Value(*Arg, TEXT("Genesis="), GenesisIndex))
					{
						if (NavigationIndex >= 0 && NavigationIndex <= 2)
						{
							Genesis = static_cast<ENavigationGenesis>(GenesisIndex);
						}
					}
				}

				if (WidgetPtr)
				{
					TSharedRef<const SWidget> WidgetShared = WidgetPtr->AsShared();
					FWidgetPath WidgetPath;
					if (FSlateApplication::Get().FindPathToWidget(WidgetShared, WidgetPath))
					{
						FSlateNavigationEventSimulator::FSimulationResult Result = FSlateReflectorModule::GetModulePtr()->GetNavigationEventSimulator()->Simulate(WidgetPath, UserIndex, Genesis, UINavigation);
						if (Result.NavigationSource.IsValid())
						{
							Ar.Logf(TEXT("Source: %s [%0xd]")
								, *FWidgetReflectorNodeUtils::GetWidgetTypeAndShortName(Result.NavigationSource.GetLastWidget()).ToString()
								, FWidgetReflectorNodeUtils::GetWidgetAddress(Result.NavigationSource.GetLastWidget()));
						}
						if (Result.NavigationDestination.IsValid())
						{
							Ar.Logf(TEXT("Destination: %s [%0llx]")
								, *FWidgetReflectorNodeUtils::GetWidgetTypeAndShortName(Result.NavigationDestination.GetLastWidget()).ToString()
								, FWidgetReflectorNodeUtils::GetWidgetAddress(Result.NavigationDestination.GetLastWidget()));
						}
					}
				}
			}),
		ECVF_Cheat);
}

FText FSlateNavigationEventSimulator::ToText(ENavigationStyle NavigationStyle)
{
	switch(NavigationStyle)
	{
	case ENavigationStyle::FourCardinalDirections: return LOCTEXT("ENavigationStyle_FourCardinalDirections", "FourCardinalDirections");
	case ENavigationStyle::ConceptualNextAndPrevious: return LOCTEXT("ENavigationStyle_ConceptualNextAndPrevious", "ConceptualNextAndPrevious");
	}
	return FText::GetEmpty();
}

FText FSlateNavigationEventSimulator::ToText(ERoutedReason RoutedReason)
{
	switch (RoutedReason)
	{
	case ERoutedReason::BoundaryRule: return LOCTEXT("ERoutedReason_BoundaryRule", "BoundaryRule");
	case ERoutedReason::Window: return LOCTEXT("ERoutedReason_Window", "Window");
	case ERoutedReason::LastWidget: return LOCTEXT("ERoutedReason_LastWidget", "LastWidget");
	}
	return FText::GetEmpty();
}

FSlateNavigationEventSimulator::FSimulatedReply::FSimulatedReply(const FNavigationReply& NavigationReply)
	: EventHandler(NavigationReply.GetHandler())
	, FocusRecipient(NavigationReply.GetFocusRecipient())
	, BoundaryRule(NavigationReply.GetBoundaryRule())
{
}

TArray<FSlateNavigationEventSimulator::FSimulationResult> FSlateNavigationEventSimulator::SimulateForEachWidgets(const FWidgetPath& WidgetPath, int32 UserIndex, ENavigationGenesis Genesis, ENavigationStyle NavigationStyle)
{
	TArray<FSimulationResult> Result;
	if (!WidgetPath.IsValid())
	{
		return Result;
	}

	if (NavigationStyle == ENavigationStyle::ConceptualNextAndPrevious)
	{
		for (EUINavigation Navigation : SlateNavigationSimulationInternal::NextPrevious)
		{
			Result.Append(SimulateForEachWidgets(WidgetPath, UserIndex, Genesis, Navigation));
		}
	}
	else
	{
		for (EUINavigation Navigation : SlateNavigationSimulationInternal::FourCardinalDirections)
		{
			Result.Append(SimulateForEachWidgets(WidgetPath, UserIndex, Genesis, Navigation));
		}
	}
	return Result;
}

TArray<FSlateNavigationEventSimulator::FSimulationResult> FSlateNavigationEventSimulator::SimulateForEachWidgets(const TSharedRef<SWindow>& Window, int32 UserIndex, ENavigationGenesis Genesis, ENavigationStyle NavigationStyle)
{
	FWidgetPath WidgetPath;
	FSlateApplication::Get().FindPathToWidget(Window, WidgetPath);
	return SimulateForEachWidgets(WidgetPath, UserIndex, Genesis, NavigationStyle);
}

namespace SlateNavigationSimulationInternal
{
	struct FAppendSimulateRecursive
	{
		FAppendSimulateRecursive(FSlateNavigationEventSimulator* InSelf, TArray<FSlateNavigationEventSimulator::FSimulationResult>& InResult, int32 InUserIndex, ENavigationGenesis InGenesis, EUINavigation InNavigation)
			: Self(InSelf), Result(InResult), UserIndex(InUserIndex), Genesis(InGenesis), Navigation(InNavigation)
		{}

		FSlateNavigationEventSimulator* Self;
		TArray<FSlateNavigationEventSimulator::FSimulationResult>& Result;
		int32 UserIndex;
		ENavigationGenesis Genesis;
		EUINavigation Navigation;

		void Execute(FWidgetPath& CurWidgetPath, const TSharedRef<SWidget>& CurWidget)
		{
			if (CurWidget->SupportsKeyboardFocus())
			{
				Result.Add(Self->Simulate(CurWidgetPath, UserIndex, Genesis, Navigation));
			}

			if (FChildren* Children = CurWidget->GetChildren())
			{
				for (int32 ChildIndex = 0; ChildIndex < Children->Num(); ++ChildIndex)
				{
					TSharedRef<SWidget> ChildWidget = Children->GetChildAt(ChildIndex);
					if (ChildWidget->GetVisibility().IsVisible() && ChildWidget->IsEnabled() && CurWidget->ValidatePathToChild(&ChildWidget.Get()))
					{
						CurWidgetPath.Widgets.AddWidget({ ChildWidget, ChildWidget->GetCachedGeometry() });
						Execute(CurWidgetPath, ChildWidget);
						CurWidgetPath.Widgets.Remove(CurWidgetPath.Widgets.GetInternalArray().Num() - 1);
					}
				}
			}
		}
	};
}

TArray<FSlateNavigationEventSimulator::FSimulationResult> FSlateNavigationEventSimulator::SimulateForEachWidgets(const FWidgetPath& WidgetPath, int32 UserIndex, ENavigationGenesis Genesis, EUINavigation Navigation)
{
	TArray<FSimulationResult> Result;
	if (!WidgetPath.IsValid())
	{
		return Result;
	}

	SlateNavigationSimulationInternal::FAppendSimulateRecursive Recursive { this, Result, UserIndex, Genesis, Navigation };
	FWidgetPath CurWidgetPath = WidgetPath;
	Recursive.Execute(CurWidgetPath, WidgetPath.GetLastWidget());

	return Result;
}

TArray<FSlateNavigationEventSimulator::FSimulationResult> FSlateNavigationEventSimulator::SimulateForEachWidgets(const TSharedRef<SWindow>& Window, int32 UserIndex, ENavigationGenesis Genesis, EUINavigation Navigation)
{
	FWidgetPath WidgetPath;
	FSlateApplication::Get().FindPathToWidget(Window, WidgetPath);
	return SimulateForEachWidgets(WidgetPath, UserIndex, Genesis, Navigation);
}

TArray<FSlateNavigationEventSimulator::FSimulationResult> FSlateNavigationEventSimulator::Simulate(const FWidgetPath& WidgetPath, int32 UserIndex, ENavigationGenesis Genesis, ENavigationStyle NavigationStyle)
{
	TArray<FSimulationResult> Result;
	if (!WidgetPath.IsValid())
	{
		return Result;
	}

	if (NavigationStyle == ENavigationStyle::ConceptualNextAndPrevious)
	{
		for (EUINavigation Navigation : SlateNavigationSimulationInternal::NextPrevious)
		{
			Result.Add(Simulate(WidgetPath, UserIndex, Genesis, Navigation));
		}
	}
	else
	{
		for (EUINavigation Navigation : SlateNavigationSimulationInternal::FourCardinalDirections)
		{
			Result.Add(Simulate(WidgetPath, UserIndex, Genesis, Navigation));
		}
	}

	return MoveTemp(Result);
}

FSlateNavigationEventSimulator::FSimulationResult FSlateNavigationEventSimulator::Simulate(const FWidgetPath& WidgetPath, int32 UserIndex, ENavigationGenesis Genesis, EUINavigation Navigation)
{
	FSimulationResult Result;
	if (!WidgetPath.IsValid())
	{
		return Result;
	}

	if (Navigation == EUINavigation::Num || Navigation == EUINavigation::Invalid)
	{
		return Result;
	}

	FModifierKeysState DefaultModifierKeyState;

	TSharedRef<SWindow> NavigationWindow = WidgetPath.GetDeepestWindow();
	FNavigationEvent NavigationEvent(DefaultModifierKeyState, UserIndex, Navigation, Genesis);

	for (int32 WidgetIndex = WidgetPath.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
	{
		const FArrangedWidget& CurrentArrangedWidget = WidgetPath.Widgets[WidgetIndex];
		if (CurrentArrangedWidget.Widget->IsEnabled())
		{
			FNavigationReply NavigationReply;
#if UE_WITH_SLATE_SIMULATEDNAVIGATIONMETADATA
			if (TSharedPtr<FSimulatedNavigationMetaData> SimulationMetaData = CurrentArrangedWidget.Widget->GetMetaData<FSimulatedNavigationMetaData>())
			{
				if (SimulationMetaData->IsOnNavigationConst())
				{
					NavigationReply = CurrentArrangedWidget.Widget->OnNavigation(CurrentArrangedWidget.Geometry, NavigationEvent)
						.SetHandler(CurrentArrangedWidget.Widget);
				}
				else
				{
					EUINavigation Type = NavigationEvent.GetNavigationType();
					NavigationReply = FNavigationReply(SimulationMetaData->GetBoundaryRule(Type), SimulationMetaData->GetFocusRecipient(Type).Pin(), FNavigationDelegate())
						.SetHandler(CurrentArrangedWidget.Widget);
				}
			}
			else
#endif // UE_WITH_SLATE_SIMULATEDNAVIGATIONMETADATA
			{
				NavigationReply = CurrentArrangedWidget.Widget->SWidget::OnNavigation(CurrentArrangedWidget.Geometry, NavigationEvent)
					.SetHandler(CurrentArrangedWidget.Widget);
			}
			if (NavigationReply.GetBoundaryRule() != EUINavigationRule::Escape || CurrentArrangedWidget.Widget == NavigationWindow || WidgetIndex == 0)
			{
				Result = InterpretResult(WidgetPath, NavigationEvent, NavigationReply, CurrentArrangedWidget);
				if (CurrentArrangedWidget.Widget == NavigationWindow)
				{
					Result.RoutedReason = ERoutedReason::Window;
				}
				else if (WidgetIndex == 0)
				{
					Result.RoutedReason = ERoutedReason::LastWidget;
				}
				else
				{
					Result.RoutedReason = ERoutedReason::BoundaryRule;
				}
				break;
			}
		}
	}
	return Result;
}

FSlateNavigationEventSimulator::FSimulationResult FSlateNavigationEventSimulator::InterpretResult(const FWidgetPath& NavigationSource, const FNavigationEvent& NavigationEvent, const FNavigationReply& NavigationReply, const FArrangedWidget& BoundaryWidget)
{
	FSimulationResult Result;

	Result.NavigationSource = NavigationSource;
	Result.NavigationType = NavigationEvent.GetNavigationType();
	Result.NavigationReply = NavigationReply;
	Result.bRoutedHandlerHasNavigationMeta = NavigationReply.GetHandler()->GetMetaData<FNavigationMetaData>().IsValid();

	if (NavigationReply.GetBoundaryRule() == EUINavigationRule::Explicit)
	{
		Result.bIsDynamic = true;
		Result.WidgetThatShouldReceivedFocus = NavigationReply.GetFocusRecipient();
		Result.bAlwaysHandleNavigationAttempt = true;
	}
	else if (NavigationReply.GetBoundaryRule() == EUINavigationRule::Custom)
	{
		Result.bIsDynamic = true;
		Result.bAlwaysHandleNavigationAttempt = true;
	}
	else if (NavigationReply.GetBoundaryRule() == EUINavigationRule::CustomBoundary)
	{
		// Will do a HittestGrid and return the result of a delegate if the HittestGrid is successful
		Result.bIsDynamic = true;
	}
	else if (NavigationEvent.GetNavigationType() == EUINavigation::Next || NavigationEvent.GetNavigationType() == EUINavigation::Previous)
	{
		FWeakWidgetPath WeakNavigationSource(NavigationSource);

		FWidgetPath NewFocusedWidgetPath = WeakNavigationSource.ToNextFocusedPath(NavigationEvent.GetNavigationType(), NavigationReply, BoundaryWidget);
		Result.WidgetThatShouldReceivedFocus = NewFocusedWidgetPath.Widgets.Last().Widget;
	}
	else
	{
		// Resolve the Widget Path and switch worlds for widgets in the current path 
		const FArrangedWidget& FocusedArrangedWidget = NavigationSource.Widgets.Last();
		FScopedSwitchWorldHack SwitchWorld(NavigationSource);

		Result.WidgetThatShouldReceivedFocus = NavigationSource.GetDeepestWindow()->GetHittestGrid().FindNextFocusableWidget(FocusedArrangedWidget, NavigationEvent.GetNavigationType(), NavigationReply, BoundaryWidget, NavigationEvent.GetUserIndex());
	}

	if (Result.WidgetThatShouldReceivedFocus)
	{
		if (OnViewportHandleNavigation.IsBound())
		{
			if (TSharedPtr<ISlateViewport> Viewport = NavigationSource.GetWindow()->GetViewport())
			{
				if (TSharedPtr<SWidget> ViewportWidget = Viewport->GetWidget().Pin())
				{
					if (NavigationSource.ContainsWidget(ViewportWidget.Get()))
					{
						TOptional<FWidgetPath> ViewportResult = OnViewportHandleNavigation.Execute(NavigationEvent.GetUserIndex(), Result.WidgetThatShouldReceivedFocus);
						Result.bHandledByViewport = ViewportResult.IsSet();
						if (Result.bHandledByViewport)
						{
							Result.NavigationDestination = ViewportResult.GetValue();
						}
					}
				}
			}
		}

		if (!Result.bHandledByViewport)
		{
			// See FSlateApplication::SetUserFocus
			FWidgetPath PathToWidget;
			Result.bCanFindWidgetForSetFocus = FSlateWindowHelper::FindPathToWidget(FSlateApplication::Get().GetTopLevelWindows(), Result.WidgetThatShouldReceivedFocus.ToSharedRef(), PathToWidget);
			if (!Result.bCanFindWidgetForSetFocus)
			{
				Result.bCanFindWidgetForSetFocus = FSlateApplication::Get().FindPathToWidget(Result.WidgetThatShouldReceivedFocus.ToSharedRef(), PathToWidget);
			}
		
			if (Result.bCanFindWidgetForSetFocus)
			{
				// Test if that widget may have focus. 
				for (int32 WidgetIndex = PathToWidget.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
				{
					const FArrangedWidget& WidgetToFocus = PathToWidget.Widgets[WidgetIndex];
					if (WidgetToFocus.Widget->SupportsKeyboardFocus())
					{
						Result.NavigationDestination = PathToWidget.GetPathDownTo(WidgetToFocus.Widget);
						Result.FocusedWidgetPath = Result.NavigationDestination.IsValid() ? Result.NavigationDestination.GetLastWidget() : TSharedPtr<SWidget>();
						break;
					}
				}
			}
		}
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE
