// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateReflectorModule.h"
#include "Modules/ModuleManager.h"

#include "SlateNavigationEventSimulator.h"
#include "Textures/SlateIcon.h"
#include "WidgetSnapshotService.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "InputEventVisualizer.h"
#include "Styling/CoreStyle.h"
#include "Styling/WidgetReflectorStyle.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNavigationSimulationList.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SAtlasVisualizer.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWidgetReflector.h"

#define LOCTEXT_NAMESPACE "FSlateReflectorModuleImpl"


/**
 * Implements the SlateReflector module.
 */
class FSlateReflectorModuleImpl : public FSlateReflectorModule
{
public:
	virtual TSharedPtr<SWidgetReflector> GetWidgetReflectorInstance() override
	{
		return WidgetReflectorPtr.Pin();
	}

	virtual FInputEventVisualizer* GetInputEventVisualizer() override
	{
		return VisualizeInputEventsPtr.Get();
	}

	// ISlateReflectorModule interface

	TSharedRef<SWidget> GetWidgetReflector(const TSharedRef<SDockTab>& InParentTab)
	{
		TSharedPtr<SWidgetReflector> WidgetReflector = WidgetReflectorPtr.Pin();

		if (!WidgetReflector.IsValid())
		{
			WidgetReflector = SNew(SWidgetReflector)
				.ParentTab(InParentTab)
				.WidgetSnapshotService(WidgetSnapshotService);
			WidgetReflectorPtr = WidgetReflector;
			FSlateApplication::Get().SetWidgetReflector(WidgetReflector.ToSharedRef());
		}

		return WidgetReflector.ToSharedRef();
	}

	TSharedRef<SWidget> GetAtlasVisualizer( ISlateAtlasProvider* InAtlasProvider )
	{
		return SNew(SAtlasVisualizer)
			.AtlasProvider(InAtlasProvider);
	}

	TSharedRef<SWidget> GetTextureAtlasVisualizer()
	{
		ISlateAtlasProvider* const AtlasProvider = FSlateApplication::Get().GetRenderer()->GetTextureAtlasProvider();

		if( AtlasProvider )
		{
			return GetAtlasVisualizer( AtlasProvider );
		}
		else
		{
			return SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NoTextureAtlasProvider", "There is no texture atlas provider available for the current renderer."))
				];
		}
	}

	TSharedRef<SWidget> GetFontAtlasVisualizer()
	{
		ISlateAtlasProvider* const AtlasProvider = FSlateApplication::Get().GetRenderer()->GetFontAtlasProvider();

		if( AtlasProvider )
		{
			return GetAtlasVisualizer( AtlasProvider );
		}
		else
		{
			return SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NoFontAtlasProvider", "There is no font atlas provider available for the current renderer."))
				];
		}
	}

	virtual void DisplayWidgetReflector() override
	{
		check(bHasRegisteredTabSpawners);
		FGlobalTabmanager::Get()->TryInvokeTab(FTabId("WidgetReflector"));
	}

	virtual void DisplayTextureAtlasVisualizer() override
	{
		check(bHasRegisteredTabSpawners);
		FGlobalTabmanager::Get()->TryInvokeTab(FTabId("TextureAtlasVisualizer"));
	}

	virtual void DisplayFontAtlasVisualizer() override
	{
		check(bHasRegisteredTabSpawners);
		FGlobalTabmanager::Get()->TryInvokeTab(FTabId("FontAtlasVisualizer"));
	}

	virtual void RegisterTabSpawner( const TSharedPtr<FWorkspaceItem>& WorkspaceGroup ) override
	{
		if (bHasRegisteredTabSpawners)
		{
			UnregisterTabSpawner();
		}

		bHasRegisteredTabSpawners = true;

		{
			FTabSpawnerEntry& SpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner("WidgetReflector", FOnSpawnTab::CreateRaw(this, &FSlateReflectorModuleImpl::MakeWidgetReflectorTab) )
				.SetDisplayName(LOCTEXT("WidgetReflectorTitle", "Widget Reflector"))
				.SetTooltipText(LOCTEXT("WidgetReflectorTooltipText", "Open the Widget Reflector tab."))
				.SetIcon(FSlateIcon(FWidgetReflectorStyle::GetStyleSetName(), "WidgetReflector.TabIcon"));

			if (WorkspaceGroup.IsValid())
			{
				SpawnerEntry.SetGroup(WorkspaceGroup.ToSharedRef());
			}
		}

		{
			FTabSpawnerEntry& SpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner("TextureAtlasVisualizer", FOnSpawnTab::CreateRaw(this, &FSlateReflectorModuleImpl::MakeTextureAtlasVisualizerTab) )
				.SetDisplayName(LOCTEXT("TextureAtlasVisualizerTitle", "Texture Atlas Visualizer"))
				.SetTooltipText(LOCTEXT("TextureAtlasVisualizerTooltipText", "Open the Texture Atlas Visualizer tab."))
				.SetMenuType(ETabSpawnerMenuType::Hidden);

			if (WorkspaceGroup.IsValid())
			{
				SpawnerEntry.SetGroup(WorkspaceGroup.ToSharedRef());
			}
		}

		{
			FTabSpawnerEntry& SpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner("FontAtlasVisualizer", FOnSpawnTab::CreateRaw(this, &FSlateReflectorModuleImpl::MakeFontAtlasVisualizerTab) )
				.SetDisplayName(LOCTEXT("FontAtlasVisualizerTitle", "Font Atlas Visualizer"))
				.SetTooltipText(LOCTEXT("FontAtlasVisualizerTooltipText", "Open the Font Atlas Visualizer tab."))
				.SetMenuType(ETabSpawnerMenuType::Hidden);

			if (WorkspaceGroup.IsValid())
			{
				SpawnerEntry.SetGroup(WorkspaceGroup.ToSharedRef());
			}
		}
	}

	virtual void UnregisterTabSpawner() override
	{
		bHasRegisteredTabSpawners = false;

		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner("WidgetReflector");
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner("TextureAtlasVisualizer");
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner("FontAtlasVisualizer");
	}

	virtual FSlateNavigationEventSimulator* GetNavigationEventSimulator() const override
	{
		return NavigationEventSimulator.Get();
	}

	virtual TSharedRef<INavigationEventSimulationView> CreateNavigationEventSimulationView(const FNavigationEventSimulationViewArgs& DetailsViewArgs) override
	{
		return SNew(SNavigationSimulationLiveList, DetailsViewArgs.OnWidgetSelected, DetailsViewArgs.OnNavigateToWidget);
	}

public:

	// IModuleInterface interface

	virtual void StartupModule() override
	{
		FWidgetReflectorStyle::Initialize();
		WidgetSnapshotService = MakeShared<FWidgetSnapshotService>();
		NavigationEventSimulator = MakeUnique<FSlateNavigationEventSimulator>();
		VisualizeInputEventsPtr = MakeUnique<FInputEventVisualizer>();

		bHasRegisteredTabSpawners = false;
		RegisterTabSpawner(nullptr);
	}

	virtual void ShutdownModule() override
	{
		UnregisterTabSpawner();

		VisualizeInputEventsPtr.Reset();
		NavigationEventSimulator.Reset();
		WidgetSnapshotService.Reset();
		FWidgetReflectorStyle::Shutdown();
	}

private:

	TSharedRef<SDockTab> MakeWidgetReflectorTab( const FSpawnTabArgs& )
	{
		TSharedRef<SDockTab> WidgetReflectorTab = SNew(SDockTab)
			.TabRole(ETabRole::NomadTab);
		WidgetReflectorTab->SetContent(GetWidgetReflector(WidgetReflectorTab));
		return WidgetReflectorTab;
	}

	TSharedRef<SDockTab> MakeTextureAtlasVisualizerTab( const FSpawnTabArgs& )
	{
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				GetTextureAtlasVisualizer()
			];
	}

	TSharedRef<SDockTab> MakeFontAtlasVisualizerTab( const FSpawnTabArgs& )
	{
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				GetFontAtlasVisualizer()
			];
	}

private:

	/** True if the tab spawners have been registered for this module */
	bool bHasRegisteredTabSpawners;

	/** Holds the widget reflector singleton. */
	TWeakPtr<SWidgetReflector> WidgetReflectorPtr;

	/** The service for handling remote widget snapshots */
	TSharedPtr<FWidgetSnapshotService> WidgetSnapshotService;

	/** NavigationEventSimulator instance */
	TUniquePtr<FSlateNavigationEventSimulator> NavigationEventSimulator;

	/** Holds the visualize input event singleton. */
	TUniquePtr<FInputEventVisualizer> VisualizeInputEventsPtr;
};


IMPLEMENT_MODULE(FSlateReflectorModuleImpl, SlateReflector);


#undef LOCTEXT_NAMESPACE
