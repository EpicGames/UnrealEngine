// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "ISlateReflectorModule.h"

class FInputEventVisualizer;
class SWidgetReflector;

/**
 * Implements the SlateReflector module.
 */
class FSlateReflectorModule : public ISlateReflectorModule
{
public:	
	/** Get this module */
	static FSlateReflectorModule* GetModulePtr()
	{
		static const FName ModuleName = "SlateReflector";
		return FModuleManager::GetModulePtr<FSlateReflectorModule>(ModuleName);
	}
	
	/** Get the current instance of the WidgetReflector */
	virtual TSharedPtr<SWidgetReflector> GetWidgetReflectorInstance() = 0;

	/** Get the current instance of the Input Event Visualizer */
	virtual FInputEventVisualizer* GetInputEventVisualizer() = 0;
};
