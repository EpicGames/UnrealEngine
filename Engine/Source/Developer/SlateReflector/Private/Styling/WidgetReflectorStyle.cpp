// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/WidgetReflectorStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/ToolBarStyle.h"

// This is to fix the issue that SlateStyleMacros like IMAGE_BRUSH look for RootToContentDir but StyleSet->RootToContentDir is how this style is set up
#define RootToContentDir StyleSet->RootToContentDir
#define RootToCoreContentDir StyleSet->RootToCoreContentDir

TSharedPtr< FSlateStyleSet > FWidgetReflectorStyle::StyleInstance = nullptr;

void FWidgetReflectorStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FWidgetReflectorStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FWidgetReflectorStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("WidgetReflectorStyleStyle"));
	return StyleSetName;
}

TSharedRef< FSlateStyleSet > FWidgetReflectorStyle::Create()
{
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);

	TSharedRef<FSlateStyleSet> StyleSet = MakeShareable(new FSlateStyleSet(FWidgetReflectorStyle::GetStyleSetName()));
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	{
#if SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM
		if (!FPlatformProperties::RequiresCookedData())
		{
			StyleSet->Set("Icon.FocusPicking", new IMAGE_BRUSH("Icons/SlateReflector/FocusPicking_24x", Icon24x24));
			StyleSet->Set("Icon.HitTestPicking", new IMAGE_BRUSH("Icons/GeneralTools/Select_40x", Icon24x24));
			StyleSet->Set("Icon.VisualPicking", new IMAGE_BRUSH("Icons/GeneralTools/Paint_40x", Icon24x24));
			StyleSet->Set("Icon.TakeSnapshot", new IMAGE_BRUSH_SVG("Starship/Common/SaveThumbnail", Icon24x24));
			StyleSet->Set("WidgetReflector.TabIcon", new IMAGE_BRUSH_SVG("Starship/Common/Widget", Icon16x16));
		}
#endif // SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM
		
		StyleSet->Set("Icon.LoadSnapshot", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Import", Icon24x24));
		StyleSet->Set("Icon.Filter", new CORE_IMAGE_BRUSH_SVG("Starship/Common/filter", Icon24x24));

		StyleSet->Set("Symbols.LeftArrow", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-left", Icon24x24));
		StyleSet->Set("Symbols.RightArrow", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-right", Icon24x24));
		StyleSet->Set("Symbols.UpArrow", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-up", Icon24x24));
		StyleSet->Set("Symbols.DownArrow", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-down", Icon24x24));		
		StyleSet->Set("Icon.Ellipsis", new CORE_IMAGE_BRUSH_SVG("Starship/Common/ellipsis-vertical-narrow", FVector2D(6, 24)));
	}

	{
		FToolBarStyle SlimToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar");
		const FTextBlockStyle& ButtonText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("ButtonText");
		SlimToolbarStyle.SetLabelStyle(FTextBlockStyle(ButtonText));
		StyleSet->Set("BoldSlimToolbar", SlimToolbarStyle);
	}

	{
		FToolBarStyle SlimToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar");
		StyleSet->Set("Menu.Label", SlimToolbarStyle.LabelStyle);
		StyleSet->Set("Menu.Button", FButtonStyle(SlimToolbarStyle.ButtonStyle).SetNormalPadding(0.0f).SetPressedPadding(0.0f));
	}

	return StyleSet;
}

const ISlateStyle& FWidgetReflectorStyle::Get()
{
	return *StyleInstance;
}

#undef RootToCoreContentDir
#undef RootToContentDir