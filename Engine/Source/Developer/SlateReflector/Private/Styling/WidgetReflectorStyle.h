// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"

/**
* Implements the visual style of the widget reflector.
*/
class FWidgetReflectorStyle
{
public:
	static void Initialize();

	static void Shutdown();

	static const ISlateStyle& Get();

	static FName GetStyleSetName();

private:
	static TSharedRef< class FSlateStyleSet > Create();

private:
	static TSharedPtr< class FSlateStyleSet > StyleInstance;
};

