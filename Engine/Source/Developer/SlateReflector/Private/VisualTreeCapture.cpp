// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualTreeCapture.h"
#include "Debugging/SlateDebugging.h"
#include "FastUpdate/SlateInvalidationRoot.h"
#include "Rendering/SlateRenderTransform.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SWidget.h"
#include "Framework/Application/SlateApplication.h"
#include "Types/InvisibleToWidgetReflectorMetaData.h"

static float VectorSign(const FVector2f& Vec, const FVector2f& A, const FVector2f& B)
{
	return FMath::Sign((B.X - A.X) * (Vec.Y - A.Y) - (B.Y - A.Y) * (Vec.X - A.X));
}

// Returns true when the point is inside the triangle
// Should not return true when the point is on one of the edges
static bool IsPointInTriangle(const FVector2f& TestPoint, const FVector2f& A, const FVector2f& B, const FVector2f& C)
{
	float BA = VectorSign(B, A, TestPoint);
	float CB = VectorSign(C, B, TestPoint);
	float AC = VectorSign(A, C, TestPoint);

	// point is in the same direction of all 3 tri edge lines
	// must be inside, regardless of tri winding
	return BA == CB && CB == AC;
}

FVisualEntry::FVisualEntry(const TWeakPtr<const SWidget>& InWidget, int32 InElementIndex, EElementType InElementType)
	: ElementIndex(InElementIndex)
	, ElementType(InElementType)
	, bFromCache(false)
	, Widget(InWidget)
{
}

FVisualEntry::FVisualEntry(const TSharedRef<const SWidget>& InWidget, const FSlateDrawElement& InElement)
{
	const FSlateRenderTransform& Transform = InElement.GetRenderTransform();
	const FVector2f LocalSize = InElement.GetLocalSize();

	TopLeft = Transform.TransformPoint(FVector2f(0.0f, 0.0f));
	TopRight = Transform.TransformPoint(FVector2f(LocalSize.X, 0.0f));
	BottomLeft = Transform.TransformPoint(FVector2f(0.0f, LocalSize.Y));
	BottomRight = Transform.TransformPoint(LocalSize);

	LayerId = InElement.GetLayer();
	ClippingIndex = INDEX_NONE;

	bFromCache = true;
	Widget = InWidget;
}

void FVisualEntry::Resolve(const FSlateWindowElementList& ElementList)
{
	if (bFromCache)
	{
		return;
	}

	auto ResolveBounds = [&](const auto& Container, uint8 InElementType)
	{
		if (InElementType == (uint8)ElementType)
		{
			const FSlateDrawElement& Element = Container[ElementIndex];
			const FSlateRenderTransform& Transform = Element.GetRenderTransform();
			const FVector2f LocalSize = Element.GetLocalSize();

			TopLeft = Transform.TransformPoint(FVector2f(0.0f, 0.0f));
			TopRight = Transform.TransformPoint(FVector2f(LocalSize.X, 0.0f));
			BottomLeft = Transform.TransformPoint(FVector2f(0.0f, LocalSize.Y));
			BottomRight = Transform.TransformPoint(LocalSize);

			LayerId = Element.GetLayer();
			ClippingIndex = Element.GetPrecachedClippingIndex();
		}
	};

	VisitTupleElements(ResolveBounds, ElementList.GetUncachedDrawElements(), UE::Slate::MakeTupleIndicies<uint8, (uint8)EElementType::ET_Count>());
}

bool FVisualEntry::IsPointInside(const FVector2f& Point) const
{
	if (IsPointInTriangle(Point, TopLeft, TopRight, BottomLeft) || IsPointInTriangle(Point, BottomLeft, TopRight, BottomRight))
	{
		return true;
	}

	return false;
}

TSharedPtr<const SWidget> FVisualTreeSnapshot::Pick(FVector2f Point) const
{
	for (int32 Index = Entries.Num() - 1; Index >= 0; Index--)
	{
		const FVisualEntry& Entry = Entries[Index];
		if (Entry.ClippingIndex != INDEX_NONE)
		{
			const TArray<FSlateClippingState>& LocalClippingState = Entry.bFromCache ? CachedClippingStates : ClippingStates;
			if (ensure(LocalClippingState.IsValidIndex(Entry.ClippingIndex)))
			{
				if (!LocalClippingState[Entry.ClippingIndex].IsPointInside(Point))
				{
					continue;
				}
			}
		}

		if (!Entry.IsPointInside(Point))
		{
			continue;
		}

		return Entry.Widget.Pin();
	}

	return TSharedPtr<const SWidget>();
}

TArray<int32> FVisualTreeSnapshot::PickAll(FVector2f Point) const
{
	TArray<int32> Result;
	for (int Index = Entries.Num() - 1; Index >= 0; Index--)
	{
		const FVisualEntry& Entry = Entries[Index];
		if (Entry.IsPointInside(Point))
		{
			if (Entry.ClippingIndex != INDEX_NONE)
			{
				const TArray<FSlateClippingState>& LocalClippingState = Entry.bFromCache ? CachedClippingStates : ClippingStates;
				if (ensure(LocalClippingState.IsValidIndex(Entry.ClippingIndex)))
				{
					if (LocalClippingState[Entry.ClippingIndex].IsPointInside(Point))
					{
						Result.Add(Index);
					}
				}
			}
			else
			{
				Result.Add(Index);
			}
		}
	}

	return Result;
}

FVisualTreeCapture::FVisualTreeCapture()
	: bIsEnabled(false)
	, WindowIsInvalidationRootCounter(0)
	, WidgetIsInvalidationRootCounter(0)
	, WidgetIsInvisibleToWidgetReflectorCounter(0)
{
}

FVisualTreeCapture::~FVisualTreeCapture()
{
	Disable();
}

void FVisualTreeCapture::Enable()
{
#if WITH_SLATE_DEBUGGING
	if (ensure(bIsEnabled == false))
	{
		FSlateApplication::Get().OnWindowBeingDestroyed().AddRaw(this, &FVisualTreeCapture::OnWindowBeingDestroyed);
		FSlateDebugging::BeginWindow.AddRaw(this, &FVisualTreeCapture::BeginWindow);
		FSlateDebugging::EndWindow.AddRaw(this, &FVisualTreeCapture::EndWindow);
		FSlateDebugging::BeginWidgetPaint.AddRaw(this, &FVisualTreeCapture::BeginWidgetPaint);
		FSlateDebugging::EndWidgetPaint.AddRaw(this, &FVisualTreeCapture::EndWidgetPaint);
		FSlateDebugging::ElementTypeAdded.AddRaw(this, &FVisualTreeCapture::ElementTypeAdded);
		bIsEnabled = true;

		WindowIsInvalidationRootCounter = 0;
		WidgetIsInvalidationRootCounter = 0;
		WidgetIsInvisibleToWidgetReflectorCounter = 0;
	}
#endif
}

void FVisualTreeCapture::Disable()
{
#if WITH_SLATE_DEBUGGING
	if (bIsEnabled)
	{
		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().OnWindowBeingDestroyed().RemoveAll(this);
		}
		FSlateDebugging::BeginWindow.RemoveAll(this);
		FSlateDebugging::EndWindow.RemoveAll(this);
		FSlateDebugging::BeginWidgetPaint.RemoveAll(this);
		FSlateDebugging::EndWidgetPaint.RemoveAll(this);
		FSlateDebugging::ElementTypeAdded.RemoveAll(this);
		bIsEnabled = false;
	}
#endif
}

void FVisualTreeCapture::Reset()
{
	VisualTrees.Reset();
}

TSharedPtr<const FVisualTreeSnapshot> FVisualTreeCapture::GetVisualTreeForWindow(SWindow* InWindow) const
{
	return VisualTrees.FindRef(InWindow);
}

void FVisualTreeCapture::AddInvalidationRootCachedEntries(TSharedRef<FVisualTreeSnapshot> Tree, const FSlateInvalidationRoot* InvalidationRoot)
{
	check(InvalidationRoot);
	const FSlateCachedElementData& Data = InvalidationRoot->GetCachedElements();
	const TArray<TSharedPtr<FSlateCachedElementList>>& CachedElements = Data.GetCachedElementLists();
	for (const TSharedPtr<FSlateCachedElementList>& CachedElement : CachedElements)
	{
		auto AddTypedEntries = [&](auto& Container)
		{
			const SWidget* Widget = CachedElement->OwningWidget;
			// todo, should check if parents has the metadata also
			if (Widget && !Widget->GetMetaData<FInvisibleToWidgetReflectorMetaData>())
			{
				for (const FSlateDrawElement& Element : Container)
				{
					const int32 EntryIndex = Tree->Entries.Emplace(Widget->AsShared(), Element);

					const FSlateClippingState* ClippingState = Element.GetClippingHandle().GetCachedClipState();
					if (ClippingState)
					{
						int32& ClippingRefIndex = Tree->Entries[EntryIndex].ClippingIndex;
						ClippingRefIndex = Tree->CachedClippingStates.IndexOfByKey(*ClippingState);
						if (ClippingRefIndex == INDEX_NONE)
						{
							ClippingRefIndex = Tree->CachedClippingStates.Add(*ClippingState);
						}
					}
				}
			}
		};

		VisitTupleElements(AddTypedEntries, CachedElement->DrawElements);
	}
}

void FVisualTreeCapture::BeginWindow(const FSlateWindowElementList& ElementList)
{
	TSharedPtr<FVisualTreeSnapshot> Tree = VisualTrees.FindRef(ElementList.GetPaintWindow());
	if (!Tree.IsValid())
	{
		Tree = MakeShared<FVisualTreeSnapshot>();
		VisualTrees.Add(ElementList.GetPaintWindow(), Tree);
	}

	Tree->Entries.Reset();
	Tree->ClippingStates.Reset();
	Tree->CachedClippingStates.Reset();

	if (ElementList.GetPaintWindow()->Advanced_IsInvalidationRoot())
	{
		++WindowIsInvalidationRootCounter;
	}
}

void FVisualTreeCapture::EndWindow(const FSlateWindowElementList& ElementList)
{
	if (ElementList.GetPaintWindow()->Advanced_IsInvalidationRoot())
	{
		--WindowIsInvalidationRootCounter;
	}

	TSharedPtr<FVisualTreeSnapshot> Tree = VisualTrees.FindRef(ElementList.GetPaintWindow());
	if (Tree.IsValid())
	{
		for (FVisualEntry& Entry : Tree->Entries)
		{
			Entry.Resolve(ElementList);
		}

		if (ElementList.GetPaintWindow()->Advanced_IsInvalidationRoot())
		{
			// Add cached elements
			const FSlateInvalidationRoot* InvalidationRoot = ElementList.GetPaintWindow()->Advanced_AsInvalidationRoot();
			AddInvalidationRootCachedEntries(Tree.ToSharedRef(), InvalidationRoot);
		}

		Tree->ClippingStates = ElementList.GetClippingManager().GetClippingStates();
		Tree->Entries.Sort([](const FVisualEntry& A, const FVisualEntry& B) {
			return A.LayerId < B.LayerId;
		});
	}
}

void FVisualTreeCapture::BeginWidgetPaint(const SWidget* Widget, const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, const FSlateWindowElementList& ElementList, int32 LayerId)
{
	TSharedPtr<FVisualTreeSnapshot> Tree = VisualTrees.FindRef(ElementList.GetPaintWindow());
	if (Tree.IsValid())
	{
		Tree->WidgetStack.Push(Widget->AsShared());

		if (Widget->Advanced_IsInvalidationRoot())
		{
			++WidgetIsInvalidationRootCounter;
		}
		if (Widget->GetMetaData<FInvisibleToWidgetReflectorMetaData>())
		{
			++WidgetIsInvisibleToWidgetReflectorCounter;
		}
	}
}

void FVisualTreeCapture::EndWidgetPaint(const SWidget* Widget, const FSlateWindowElementList& ElementList, int32 LayerId)
{
	TSharedPtr<FVisualTreeSnapshot> Tree = VisualTrees.FindRef(ElementList.GetPaintWindow());
	if (Tree.IsValid())
	{
		Tree->WidgetStack.Pop();

		if (Widget->Advanced_IsInvalidationRoot())
		{
			--WidgetIsInvalidationRootCounter;

			// Add cached elements
			const FSlateInvalidationRoot* InvalidationRoot = Widget->Advanced_AsInvalidationRoot();
			AddInvalidationRootCachedEntries(Tree.ToSharedRef(), InvalidationRoot);
		}
		if (Widget->GetMetaData<FInvisibleToWidgetReflectorMetaData>())
		{
			--WidgetIsInvisibleToWidgetReflectorCounter;
		}
	}
}

void FVisualTreeCapture::ElementTypeAdded(const FSlateDebuggingElementTypeAddedEventArgs& ElementTypeAddedArgs)
{
	if (WindowIsInvalidationRootCounter > 0 || WidgetIsInvalidationRootCounter > 0 || WidgetIsInvisibleToWidgetReflectorCounter > 0)
	{
		return;
	}

	TSharedPtr<FVisualTreeSnapshot> Tree = VisualTrees.FindRef(ElementTypeAddedArgs.ElementList.GetPaintWindow());
	if (Tree.IsValid())
	{
		if (Tree->WidgetStack.Num() > 0)
		{
			// Ignore any element added from a widget that's invisible to the widget reflector.
			Tree->Entries.Emplace(Tree->WidgetStack.Top(), ElementTypeAddedArgs.ElementIndex, ElementTypeAddedArgs.ElementType);
		}
	}
}

void FVisualTreeCapture::OnWindowBeingDestroyed(const SWindow& WindowBeingDestoyed)
{
	VisualTrees.Remove(&WindowBeingDestoyed);
}
