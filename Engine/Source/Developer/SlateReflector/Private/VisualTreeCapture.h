// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Rendering/DrawElementCoreTypes.h"

class SWindow;
class SWidget;
class FSlateDrawElement;
class FSlateClippingState;
class FSlateWindowElementList;
class FPaintArgs;
struct FGeometry;
class FSlateRect;
class FSlateInvalidationRoot;
struct FSlateDebuggingElementTypeAddedEventArgs;

class FVisualEntry
{
public:
	FVector2f TopLeft;
	FVector2f TopRight;
	FVector2f BottomLeft;
	FVector2f BottomRight;

	int32 LayerId = INDEX_NONE;
	int32 ClippingIndex = INDEX_NONE;
	int32 ElementIndex = INDEX_NONE;
	EElementType ElementType = EElementType::ET_Count;
	bool bFromCache = false;
	TWeakPtr<const SWidget> Widget;

	FVisualEntry(const TWeakPtr<const SWidget>& Widget, int32 InElementIndex, EElementType InElementType);
	FVisualEntry(const TSharedRef<const SWidget>& Widget, const FSlateDrawElement& InElement);

	void Resolve(const FSlateWindowElementList& ElementList);

	bool IsPointInside(const FVector2f& Point) const;
};

class FVisualTreeSnapshot : public TSharedFromThis<FVisualTreeSnapshot>
{
public:
	TSharedPtr<const SWidget> Pick(FVector2f Point) const;
	/** Returns all the the Entires at the location. */
	TArray<int32> PickAll(FVector2f Point) const;
	
public:
	TArray<FVisualEntry> Entries;
	TArray<FSlateClippingState> ClippingStates;
	TArray<FSlateClippingState> CachedClippingStates;
	TArray<TWeakPtr<const SWidget>> WidgetStack;
};

class FVisualTreeCapture
{
public:
	FVisualTreeCapture();
	~FVisualTreeCapture();

	/** Enables visual tree capture */
	void Enable();

	/** Disables visual tree capture */
	void Disable();

	/** Resets the visual tree capture to a pre-capture state and destroys the cached visual tree captured last. */
	void Reset();

	/** The visual tree capture is enabled. */
	bool IsEnabled() const
	{
		return bIsEnabled;
	}

	/** The visual tree capture contains elements. */
	bool HasElements() const
	{
		return VisualTrees.Num() > 0;
	}

	TSharedPtr<const FVisualTreeSnapshot> GetVisualTreeForWindow(SWindow* InWindow) const;
	
private:

	void AddInvalidationRootCachedEntries(TSharedRef<FVisualTreeSnapshot> Tree, const FSlateInvalidationRoot* Entries);


	void BeginWindow(const FSlateWindowElementList& ElementList);
	void EndWindow(const FSlateWindowElementList& ElementList);

	void BeginWidgetPaint(const SWidget* Widget, const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, const FSlateWindowElementList& ElementList, int32 LayerId);

	/**  */
	void EndWidgetPaint(const SWidget* Widget, const FSlateWindowElementList& ElementList, int32 LayerId);

	/**  */
	void ElementTypeAdded(const FSlateDebuggingElementTypeAddedEventArgs& ElementTypeAddedArgs);

	void OnWindowBeingDestroyed(const SWindow& WindowBeingDestoyed);

private:
	TMap<const SWindow*, TSharedPtr<FVisualTreeSnapshot>> VisualTrees;
	bool bIsEnabled;
	int32 WindowIsInvalidationRootCounter;
	int32 WidgetIsInvalidationRootCounter;
	int32 WidgetIsInvisibleToWidgetReflectorCounter;
};