// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "WidgetSnapshotMessages.generated.h"


/**
 * Implements a message that is used to request a widget snapshot from a remote target.
 */
USTRUCT()
struct FWidgetSnapshotRequest
{
	GENERATED_USTRUCT_BODY()

	/** The instance ID of the remote target we want to receive a snapshot from */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid TargetInstanceId;

	/** The request ID of this snapshot (used to identify the correct response from the target) */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid SnapshotRequestId;
};


/**
 * Implements a message that is used to receive a widget snapshot from a remote target.
 */
USTRUCT()
struct FWidgetSnapshotResponse
{
	GENERATED_USTRUCT_BODY()

	/** The request ID of this snapshot (used to identify the correct response from the target) */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid SnapshotRequestId;

	/** The snapshot data, to be used by FWidgetSnapshotData::LoadSnapshotFromBuffer */
	UPROPERTY(EditAnywhere, Category="Message")
	TArray<uint8> SnapshotData;
};
