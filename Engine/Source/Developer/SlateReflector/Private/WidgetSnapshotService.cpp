// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetSnapshotService.h"
#include "Misc/App.h"
#include "WidgetSnapshotMessages.h"
#include "IMessagingModule.h"
#include "MessageEndpointBuilder.h"
#include "Widgets/SWidgetSnapshotVisualizer.h"

FWidgetSnapshotService::FWidgetSnapshotService()
{
	if (FPlatformMisc::SupportsMessaging() && FPlatformProcess::SupportsMultithreading())
	{
		TSharedPtr<IMessageBus, ESPMode::ThreadSafe> MessageBusPtr = IMessagingModule::Get().GetDefaultBus();

		MessageEndpoint = FMessageEndpoint::Builder("FWidgetSnapshotService", MessageBusPtr.ToSharedRef())
			.ReceivingOnThread(ENamedThreads::GameThread)
			.Handling<FWidgetSnapshotRequest>(this, &FWidgetSnapshotService::HandleWidgetSnapshotRequestMessage)
			.Handling<FWidgetSnapshotResponse>(this, &FWidgetSnapshotService::HandleWidgetSnapshotResponseMessage);

		if (MessageEndpoint.IsValid())
		{
			MessageEndpoint->Subscribe<FWidgetSnapshotRequest>();
		}
	}
}

FGuid FWidgetSnapshotService::RequestSnapshot(const FGuid& InRemoteInstanceId, const FOnWidgetSnapshotResponse& OnResponse)
{
	if (MessageEndpoint.IsValid())
	{
		FWidgetSnapshotRequest* WidgetSnapshotRequest = FMessageEndpoint::MakeMessage<FWidgetSnapshotRequest>();
		WidgetSnapshotRequest->TargetInstanceId = InRemoteInstanceId;
		WidgetSnapshotRequest->SnapshotRequestId = FGuid::NewGuid();

		PendingSnapshotResponseHandlers.Add(WidgetSnapshotRequest->SnapshotRequestId, OnResponse);

		MessageEndpoint->Publish(WidgetSnapshotRequest);

		return WidgetSnapshotRequest->SnapshotRequestId;
	}

	return FGuid();
}

void FWidgetSnapshotService::AbortSnapshotRequest(const FGuid& InSnapshotRequestId)
{
	PendingSnapshotResponseHandlers.Remove(InSnapshotRequestId);
}

void FWidgetSnapshotService::HandleWidgetSnapshotRequestMessage(const FWidgetSnapshotRequest& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (MessageEndpoint.IsValid() && Message.TargetInstanceId == FApp::GetInstanceId())
	{
		FWidgetSnapshotData SnapshotData;
		SnapshotData.TakeSnapshot(false);

		FWidgetSnapshotResponse* WidgetSnapshotResponse = FMessageEndpoint::MakeMessage<FWidgetSnapshotResponse>();
		WidgetSnapshotResponse->SnapshotRequestId = Message.SnapshotRequestId;
		SnapshotData.SaveSnapshotToBuffer(WidgetSnapshotResponse->SnapshotData);

		MessageEndpoint->Send(WidgetSnapshotResponse, Context->GetSender());
	}
}

void FWidgetSnapshotService::HandleWidgetSnapshotResponseMessage(const FWidgetSnapshotResponse& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	const FOnWidgetSnapshotResponse* FoundResponseHandler = PendingSnapshotResponseHandlers.Find(Message.SnapshotRequestId);
	if (FoundResponseHandler && FoundResponseHandler->IsBound())
	{
		FoundResponseHandler->Execute(Message.SnapshotData);
		PendingSnapshotResponseHandlers.Remove(Message.SnapshotRequestId);
	}
}
