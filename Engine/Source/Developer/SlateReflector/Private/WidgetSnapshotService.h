// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "MessageEndpoint.h"

struct FWidgetSnapshotRequest;
struct FWidgetSnapshotResponse;

/**
 * Implements the service for handling remote widget snapshots.
 */
class FWidgetSnapshotService
{
public:
	DECLARE_DELEGATE_OneParam(FOnWidgetSnapshotResponse, const TArray<uint8>& /*InSnapshotData*/);

	FWidgetSnapshotService();

	/** Request a snapshot from the given instance. The given delegate will be called when the response comes in */
	FGuid RequestSnapshot(const FGuid& InRemoteInstanceId, const FOnWidgetSnapshotResponse& OnResponse);

	/** Abort a request using the GUID previously obtained via a call to RequestSnapshot */
	void AbortSnapshotRequest(const FGuid& InSnapshotRequestId);

private:
	/** Handles FWidgetSnapshotRequest messages */
	void HandleWidgetSnapshotRequestMessage(const FWidgetSnapshotRequest& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles FWidgetSnapshotResponse messages */
	void HandleWidgetSnapshotResponseMessage(const FWidgetSnapshotResponse& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Holds the message endpoint */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Handlers awaiting their resultant snapshot data */
	TMap<FGuid, FOnWidgetSnapshotResponse> PendingSnapshotResponseHandlers;
};
