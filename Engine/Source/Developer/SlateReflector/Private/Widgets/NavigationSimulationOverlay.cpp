// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/NavigationSimulationOverlay.h"

#include "Framework/Application/SlateApplication.h"
#include "Models/WidgetReflectorNode.h"
#include "Rendering/DrawElements.h"
#include "Styling/CoreStyle.h"
#include "Styling/WidgetReflectorStyle.h"


#define LOCTEXT_NAMESPACE "SlateNavigationSimulationOverlay"


//***********************************************************
//SSlateNavigationSimulationOverlay

namespace NavigationSimulationOverlay
{
	static const FName FocusRectangleBrush = TEXT("FocusRectangle");
	static const FName DebugBorderBrush = TEXT("Debug.Border");
	static const FName SymbolsDownArrow = TEXT("Symbols.DownArrow");
	static const FName SymbolsUpArrow = TEXT("Symbols.UpArrow");
	static const FName SymbolsRightArrow = TEXT("Symbols.RightArrow");
	static const FName SymbolsLeftArrow = TEXT("Symbols.LeftArrow");

	struct FOverlayBrushes
	{
		FOverlayBrushes()
		{
			const ISlateStyle& CoreStyle = FCoreStyle::Get();
			FocusBrush = CoreStyle.GetBrush(FocusRectangleBrush);
			DebugBrush = CoreStyle.GetBrush(DebugBorderBrush);

			const ISlateStyle& ReflectorStyle = FWidgetReflectorStyle::Get();
			LeftArrowBrush = ReflectorStyle.GetBrush(NavigationSimulationOverlay::SymbolsLeftArrow);
			RightArrowBrush = ReflectorStyle.GetBrush(NavigationSimulationOverlay::SymbolsRightArrow);
			UpArrowBrush = ReflectorStyle.GetBrush(NavigationSimulationOverlay::SymbolsUpArrow);
			DownArrowBrush = ReflectorStyle.GetBrush(NavigationSimulationOverlay::SymbolsDownArrow);

			ArrowSize = UpArrowBrush->ImageSize;
		}

		const FSlateBrush* FocusBrush;
		const FSlateBrush* DebugBrush;
		const FSlateBrush* LeftArrowBrush;
		const FSlateBrush* RightArrowBrush;
		const FSlateBrush* UpArrowBrush;
		const FSlateBrush* DownArrowBrush;
		FVector2D ArrowSize;
	};

	struct FPaintNode
	{
		TArray<FVector2D> SourcePoints;
		FVector2D SourceCenterPoint;
		FLinearColor Color;
		FVector2D Direction;
		const FSlateBrush* ArrowBrush = nullptr;

		static FPaintNode MakePaintNode(const FNavigationSimulationWidgetNodeItem& NodeItem, const FPaintGeometry& PaintGeometry, const FOverlayBrushes& Brushes)
		{
			FPaintNode PaintNode;

			const FVector2D TopLeft = FVector2D::ZeroVector;
			const FVector2D BottomRight = PaintGeometry.GetLocalSize();
			const FVector2D TopRight = FVector2D(BottomRight.X, TopLeft.Y);
			const FVector2D BottomLeft = FVector2D(TopLeft.X, BottomRight.Y);
			switch (NodeItem.NavigationType)
			{
			case EUINavigation::Up:
			{
				PaintNode.Direction = {0.f, 1.f};
				PaintNode.SourcePoints.Add(TopLeft);
				PaintNode.SourcePoints.Add(TopRight);
				PaintNode.SourceCenterPoint = FVector2D(TopRight.X / 2.f, TopRight.Y);
				PaintNode.Color = FLinearColor::Red;
				PaintNode.ArrowBrush = Brushes.UpArrowBrush;
				break;
			}
			case EUINavigation::Down:
			{
				PaintNode.Direction = {0.f, -1.f};
				PaintNode.SourcePoints.Add(BottomLeft);
				PaintNode.SourcePoints.Add(BottomRight);
				PaintNode.SourceCenterPoint = FVector2D(BottomRight.X / 2.f, BottomRight.Y);
				PaintNode.Color = FLinearColor::Blue;
				PaintNode.ArrowBrush = Brushes.DownArrowBrush;
				break;
			}
			case EUINavigation::Left:
			{
				PaintNode.Direction = {1.f, 0.f};
				PaintNode.SourcePoints.Add(TopLeft);
				PaintNode.SourcePoints.Add(BottomLeft);
				PaintNode.SourceCenterPoint = FVector2D(BottomLeft.X, BottomRight.Y / 2.f);
				PaintNode.Color = FColor::Cyan;
				PaintNode.ArrowBrush = Brushes.LeftArrowBrush;
				break;
			}
			case EUINavigation::Right:
			{
				PaintNode.Direction = {-1.f, 0.f};
				PaintNode.SourcePoints.Add(TopRight);
				PaintNode.SourcePoints.Add(BottomRight);
				PaintNode.SourceCenterPoint = FVector2D(BottomRight.X, BottomRight.Y / 2.f);
				PaintNode.Color = FLinearColor::Yellow;
				PaintNode.ArrowBrush = Brushes.RightArrowBrush;
			}
			break;
			}
			return PaintNode;
		}
	};

	struct FWidgetGeometryMap
	{
		TMap<FNavigationSimulationWidgetInfo::TPointerAsInt, FPaintGeometry> Map;

		const FPaintGeometry* GetGeometry(const TSharedRef<const SWidget>& Widget)
		{
			const FNavigationSimulationWidgetInfo::TPointerAsInt WidgetPtr = FWidgetReflectorNodeUtils::GetWidgetAddress(Widget);
			if (const FPaintGeometry* FoundGeometry = Map.Find(WidgetPtr))
			{
				return FoundGeometry;
			}

			TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(Widget);
			if (!WidgetWindow.IsValid())
			{
				return nullptr;
			}

			while (WidgetWindow->GetParentWidget().IsValid())
			{
				TSharedRef<SWidget> CurrentWidget = WidgetWindow->GetParentWidget().ToSharedRef();
				TSharedPtr<SWindow> ParentWidgetWindow = FSlateApplication::Get().FindWidgetWindow(CurrentWidget);
				if (!ParentWidgetWindow.IsValid())
				{
					break;
				}
				WidgetWindow = ParentWidgetWindow;
			}

			TSharedRef<SWindow> CurrentWindowRef = WidgetWindow.ToSharedRef();

			FWidgetPath WidgetPath;
			if (FSlateApplication::Get().GeneratePathToWidgetUnchecked(Widget, WidgetPath))
			{
				FArrangedWidget ArrangedWidget = WidgetPath.FindArrangedWidget(Widget).Get(FArrangedWidget::GetNullWidget());
				ArrangedWidget.Geometry.AppendTransform(FSlateLayoutTransform(Inverse(CurrentWindowRef->GetPositionInScreen())));

				const FVector2D InflateAmount = FVector2D(1, 1) / FVector2D(ArrangedWidget.Geometry.GetAccumulatedRenderTransform().GetMatrix().GetScale().GetVector());
				FPaintGeometry PaintGeometry = ArrangedWidget.Geometry.ToInflatedPaintGeometry(InflateAmount);

				Map.Add(WidgetPtr, PaintGeometry);
				return Map.Find(WidgetPtr);
			}

			return nullptr;
		}

		const FPaintGeometry* GetGeometry(const FNavigationSimulationWidgetInfo& WidgetInfo, const FGeometry& AllottedGeometry, const FVector2D& RootDrawOffset)
		{
			if (const FPaintGeometry* FoundGeometry = Map.Find(WidgetInfo.WidgetPtr))
			{
				return FoundGeometry;
			}

			if (WidgetInfo.bHasGeometry)
			{
				const FVector2D InflateAmount = FVector2D(1, 1) / FVector2D(WidgetInfo.WidgetGeometry.GetAccumulatedRenderTransform().GetMatrix().GetScale().GetVector());
				const FVector2D NewLocalOffset = RootDrawOffset + WidgetInfo.WidgetGeometry.GetAccumulatedLayoutTransform().GetTranslation();
				const FVector2D NewLocalSize = TransformPoint(WidgetInfo.WidgetGeometry.GetAccumulatedLayoutTransform().GetScale(), WidgetInfo.WidgetGeometry.GetLocalSize());

				Map.Add(WidgetInfo.WidgetPtr, AllottedGeometry.ToPaintGeometry(NewLocalSize, FSlateLayoutTransform(NewLocalOffset)));
				return Map.Find(WidgetInfo.WidgetPtr);
			}

			return nullptr;
		}
	};

	void Paint(
		const NavigationSimulationOverlay::FOverlayBrushes& Brushes
		, const FNavigationSimulationWidgetNodePtr& NodePtr
		, TFunctionRef<TOptional<FPaintGeometry>(const FNavigationSimulationWidgetInfo&)> GetGeometry
		, FSlateWindowElementList& OutDrawElements
		, int32 LayerId)
	{
		const TOptional<FPaintGeometry> SourcePaintGeometry = GetGeometry(NodePtr->NavigationSource);
		if (!SourcePaintGeometry.IsSet())
		{
			return;
		}

		const FScale2D SourceScale = SourcePaintGeometry->GetAccumulatedRenderTransform().GetMatrix().GetScale();

		for (const FNavigationSimulationWidgetNodeItem& NodeItem : NodePtr->Simulations)
		{
			const bool bCanPaint = NodeItem.NavigationType == EUINavigation::Down
				|| NodeItem.NavigationType == EUINavigation::Up
				|| NodeItem.NavigationType == EUINavigation::Left
				|| NodeItem.NavigationType == EUINavigation::Right;
			if (!NodeItem.Destination.IsWidgetExplicitlyNull() && bCanPaint)
			{
				const FPaintNode PaintNode = FPaintNode::MakePaintNode(NodeItem, SourcePaintGeometry.GetValue(), Brushes);

				FSlateDrawElement::MakeLines(
					OutDrawElements,
					LayerId,
					SourcePaintGeometry.GetValue(),
					PaintNode.SourcePoints,
					ESlateDrawEffect::None,
					PaintNode.Color,
					false,
					1.f
				);


				const TOptional<FPaintGeometry> DestinationPaintGeometry = GetGeometry(NodeItem.Destination);
				if (DestinationPaintGeometry.IsSet())
				{
					const FVector2D DestinationSize = TransformPoint(DestinationPaintGeometry->GetAccumulatedRenderTransform().GetMatrix().GetScale(), DestinationPaintGeometry->GetLocalSize());

					FVector2D DestinationCenter = DestinationPaintGeometry->GetAccumulatedRenderTransform().GetTranslation() - SourcePaintGeometry->GetAccumulatedRenderTransform().GetTranslation();
					DestinationCenter += DestinationSize / 2.f;

					FVector2D ScaledDestinationCenter = SourceScale.Inverse().TransformPoint(DestinationCenter);

					TArray<FVector2D> DestinationPoints;
					DestinationPoints.Add(PaintNode.SourceCenterPoint);
					DestinationPoints.Add(ScaledDestinationCenter);

					FSlateDrawElement::MakeLines(
						OutDrawElements,
						LayerId,
						SourcePaintGeometry.GetValue(),
						DestinationPoints,
						ESlateDrawEffect::None,
						PaintNode.Color,
						false,
						1.f
					);

					const FVector2D IconPaintSize = TransformPoint(DestinationPaintGeometry->GetAccumulatedRenderTransform().GetMatrix().GetScale(), PaintNode.ArrowBrush->ImageSize / 2.f);
					const FVector2D IconPaintOffset = TransformPoint(DestinationPaintGeometry->GetAccumulatedRenderTransform().GetMatrix().GetScale(), PaintNode.ArrowBrush->ImageSize * PaintNode.Direction / 6.f);
					const FVector2D IconPaintRootLocation = DestinationPaintGeometry->GetAccumulatedRenderTransform().GetTranslation();
					const FVector2D IconPaintLocation = IconPaintRootLocation - (IconPaintSize/2.f) + (DestinationSize / 2.f) + (IconPaintOffset);

					FGeometry ArrowGeometry = FGeometry::MakeRoot(IconPaintSize, FSlateLayoutTransform(1.f, IconPaintLocation));
					FSlateDrawElement::MakeBox(
						OutDrawElements,
						LayerId,
						ArrowGeometry.ToPaintGeometry(),
						PaintNode.ArrowBrush,
						ESlateDrawEffect::None,
						PaintNode.Color
					);
				}
			}
		}
	}
}

int32 FNavigationSimulationOverlay::PaintSnapshotNode(const TArray<FNavigationSimulationWidgetNodePtr>& NodeToPaint, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FVector2D& RootDrawOffset)
{
	++LayerId;

	NavigationSimulationOverlay::FWidgetGeometryMap GeometryMap;

	const NavigationSimulationOverlay::FOverlayBrushes OverlayBrushes;
	for (const FNavigationSimulationWidgetNodePtr& NodePtr : NodeToPaint)
	{
		auto GetGeometry = [&GeometryMap, &AllottedGeometry, RootDrawOffset](const FNavigationSimulationWidgetInfo& Info) -> TOptional<FPaintGeometry>
		{
			if (Info.bHasGeometry)
			{
				if (const FPaintGeometry* PaintGeometry = GeometryMap.GetGeometry(Info, AllottedGeometry, RootDrawOffset))
				{
					return TOptional<FPaintGeometry>(*PaintGeometry);
				}
				else
				{
					UE_LOGF(LogSlate, Warning, "Slate Navigation Simulation missing paint geometry for snapshot widget %ls", *(Info.WidgetTypeAndShortName.ToString()));
				}
			}
			return TOptional<FPaintGeometry>();
		};

		NavigationSimulationOverlay::Paint(OverlayBrushes, NodePtr, GetGeometry, OutDrawElements, LayerId);
	}
	return LayerId;
}

int32 FNavigationSimulationOverlay::PaintLiveNode(const TArray<FNavigationSimulationWidgetNodePtr>& NodeToPaint, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId)
{
	++LayerId;

	NavigationSimulationOverlay::FWidgetGeometryMap GeometryMap;

	const NavigationSimulationOverlay::FOverlayBrushes OverlayBrushes;
	for (const FNavigationSimulationWidgetNodePtr& NodePtr : NodeToPaint)
	{
		auto GetGeometry = [&GeometryMap](const FNavigationSimulationWidgetInfo& Info) -> TOptional<FPaintGeometry>
		{
			if (TSharedPtr<const SWidget> WidgetLive = Info.WidgetLive.Pin())
			{
				if (const FPaintGeometry* PaintGeometry = GeometryMap.GetGeometry(WidgetLive.ToSharedRef()))
				{
					return TOptional<FPaintGeometry>(*PaintGeometry);
				}
				else
				{
					UE_LOGF(LogSlate, Warning, "Slate Navigation Simulation missing paint geometry for live widget %ls", *(Info.WidgetTypeAndShortName.ToString()));
				}
			}
			return TOptional<FPaintGeometry>();
		};

		NavigationSimulationOverlay::Paint(OverlayBrushes, NodePtr, GetGeometry, OutDrawElements, LayerId);

	}
	return LayerId;
}

#undef LOCTEXT_NAMESPACE
