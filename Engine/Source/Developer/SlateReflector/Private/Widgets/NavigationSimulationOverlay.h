// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Models/NavigationSimulationNode.h"
#include "Widgets/SWidget.h"

/**
 * Helper class to paint the result of a navigation event simulation
 */
class FNavigationSimulationOverlay
{
public:
	FNavigationSimulationOverlay() = delete;

	static int32 PaintSnapshotNode(const TArray<FNavigationSimulationWidgetNodePtr>& NodeToPaint, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FVector2D& RootDrawOffset);
	static int32 PaintLiveNode(const TArray<FNavigationSimulationWidgetNodePtr>& NodeToPaint, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId);
};
