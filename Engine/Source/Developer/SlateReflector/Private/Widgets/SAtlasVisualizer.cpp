// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAtlasVisualizer.h"
#include "Textures/TextureAtlas.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SViewport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Framework/Layout/ScrollyZoomy.h"
#include "Styling/SlateStyleRegistry.h"
#include "Misc/FileHelper.h"

#define LOCTEXT_NAMESPACE "AtlasVisualizer"

class SAtlasVisualizerPanel : public IScrollableZoomable, public SPanel
{
public:
	using FAtlasVisualizerPanelSlot = FSingleWidgetChildrenWithSlot;

	SLATE_BEGIN_ARGS(SAtlasVisualizerPanel)
		{
			_Visibility = EVisibility::Visible;
		}
		
		SLATE_DEFAULT_SLOT( FArguments, Content )

	SLATE_END_ARGS()

	SAtlasVisualizerPanel()
		: PhysicalOffset(ForceInitToZero)
		, CachedSize(ForceInitToZero)
		, ZoomLevel(1.0f)
		, bFitToWindow(true)
		, ChildSlot(this)
		, ScrollyZoomy(false)
	{
	}

	void Construct( const FArguments& InArgs )
	{
		ChildSlot
		[
			InArgs._Content.Widget
		];
	}

	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override
	{
		CachedSize = AllottedGeometry.GetLocalSize();

		const TSharedRef<SWidget>& ChildWidget = ChildSlot.GetWidget();
		if( ChildWidget->GetVisibility() != EVisibility::Collapsed )
		{
			const FVector2f& WidgetDesiredSize = ChildWidget->GetDesiredSize();

			// Update the zoom level, and clamp the pan offset based on our current geometry
			SAtlasVisualizerPanel* const NonConstThis = const_cast<SAtlasVisualizerPanel*>(this);
			NonConstThis->UpdateFitToWindowZoom(WidgetDesiredSize, CachedSize);
			NonConstThis->ClampViewOffset(WidgetDesiredSize, CachedSize);

			ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(ChildWidget, PhysicalOffset * ZoomLevel, WidgetDesiredSize * ZoomLevel));
		}
	}

	virtual FVector2D ComputeDesiredSize(float) const override
	{
		FVector2D ThisDesiredSize = FVector2D::ZeroVector;

		const TSharedRef<SWidget>& ChildWidget = ChildSlot.GetWidget();
		if( ChildWidget->GetVisibility() != EVisibility::Collapsed )
		{
			ThisDesiredSize = ChildWidget->GetDesiredSize() * ZoomLevel;
		}

		return ThisDesiredSize;
	}

	virtual FChildren* GetChildren() override
	{
		return &ChildSlot;
	}

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override
	{
		ScrollyZoomy.Tick(InDeltaTime, *this);
	}

	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		return ScrollyZoomy.OnMouseButtonDown(MouseEvent);
	}

	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		return ScrollyZoomy.OnMouseButtonUp(AsShared(), MyGeometry, MouseEvent);
	}
		
	virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		return ScrollyZoomy.OnMouseMove(AsShared(), *this, MyGeometry, MouseEvent);
	}

	virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override
	{
		ScrollyZoomy.OnMouseLeave(AsShared(), MouseEvent);
	}
		
	virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		return ScrollyZoomy.OnMouseWheel(MouseEvent, *this);
	}

	virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override
	{
		return ScrollyZoomy.OnCursorQuery();
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		LayerId = SPanel::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
		LayerId = ScrollyZoomy.PaintSoftwareCursorIfNeeded(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
		return LayerId;
	}

	virtual bool ScrollBy( const FVector2D& Offset ) override
	{
		if( bFitToWindow )
		{
			return false;
		}

		const FVector2f PrevPhysicalOffset = PhysicalOffset;
		const float InverseZoom = 1.0f / ZoomLevel;
		PhysicalOffset += (UE::Slate::CastToVector2f(Offset) * InverseZoom);

		const TSharedRef<SWidget>& ChildWidget = ChildSlot.GetWidget();
		const FVector2f& WidgetDesiredSize = ChildWidget->GetDesiredSize();
		ClampViewOffset(WidgetDesiredSize, CachedSize);

		return PhysicalOffset != PrevPhysicalOffset;
	}

	virtual bool ZoomBy( const float Amount ) override
	{
		static const float MinZoomLevel = 0.2f;
		static const float MaxZoomLevel = 4.0f;

		bFitToWindow = false;

		const float PrevZoomLevel = ZoomLevel;
		ZoomLevel = FMath::Clamp(ZoomLevel + (Amount * 0.05f), MinZoomLevel, MaxZoomLevel);
		return ZoomLevel != PrevZoomLevel;
	}

	float GetZoomLevel() const
	{
		return ZoomLevel;
	}

	void FitToWindow()
	{
		bFitToWindow = true;
		PhysicalOffset = FVector2f::ZeroVector;
	}

	bool IsFitToWindow() const
	{
		return bFitToWindow;
	}

	void FitToSize()
	{
		bFitToWindow = false;
		ZoomLevel = 1.0f;
		PhysicalOffset = FVector2f::ZeroVector;
	}

private:
	void UpdateFitToWindowZoom( const FVector2f& ViewportSize, const FVector2f& LocalSize )
	{
		if( bFitToWindow )
		{
			const float ZoomHoriz = LocalSize.X / ViewportSize.X;
			const float ZoomVert  = LocalSize.Y / ViewportSize.Y;
			ZoomLevel = FMath::Min(ZoomHoriz, ZoomVert);
		}
	}

	void ClampViewOffset( const FVector2f& ViewportSize, const FVector2f& LocalSize )
	{
		PhysicalOffset.X = ClampViewOffsetAxis(ViewportSize.X, LocalSize.X, PhysicalOffset.X);
		PhysicalOffset.Y = ClampViewOffsetAxis(ViewportSize.Y, LocalSize.Y, PhysicalOffset.Y);
	}

	float ClampViewOffsetAxis( const float ViewportSize, const float LocalSize, const float CurrentOffset )
	{
		const float ZoomedViewportSize = ViewportSize * ZoomLevel;

		if( ZoomedViewportSize <= LocalSize )
		{
			// If the viewport is smaller than the available size, then we can't be scrolled
			return 0.0f;
		}

		// Given the size of the viewport, and the current size of the window, work how far we can scroll
		// Note: This number is negative since scrolling down/right moves the viewport up/left
		const float MaxScrollOffset = (LocalSize - ZoomedViewportSize) / ZoomLevel;

		// Clamp the left/top edge
		if( CurrentOffset < MaxScrollOffset )
		{
			return MaxScrollOffset;
		}

		// Clamp the right/bottom edge
		if( CurrentOffset > 0.0f )
		{
			return 0.0f;
		}

		return CurrentOffset;
	}

	FVector2f PhysicalOffset;
	mutable FVector2f CachedSize;
	float ZoomLevel;
	bool bFitToWindow;

	FAtlasVisualizerPanelSlot ChildSlot;
	FScrollyZoomy ScrollyZoomy;
};

void SAtlasVisualizer::Construct( const FArguments& InArgs )
{
	AtlasProvider = InArgs._AtlasProvider;
	check(AtlasProvider);

	SelectedAtlasPage = 0;
	bDisplayCheckerboard = false;

	HoveredSlotBorderBrush = FCoreStyle::Get().GetBrush("Debug.Border");

	TSharedPtr<SViewport> Viewport;
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
			.Padding(4.0f)
			.AutoHeight()
			[
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.VAlign( VAlign_Center )
				.AutoWidth()
				.Padding(0.0,2.0f,2.0f,2.0f)
				[
					SNew( STextBlock )
					.Text( LOCTEXT("SelectAPage", "Select a page") )
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f)
				.VAlign( VAlign_Center )
				[
					SAssignNew( AtlasPageCombo, SComboBox< TSharedPtr<int32> > )
					.OptionsSource( &AtlasPages )
					.OnComboBoxOpening(this, &SAtlasVisualizer::OnComboOpening)
					.OnGenerateWidget( this, &SAtlasVisualizer::OnGenerateWidgetForCombo )
					.OnSelectionChanged( this, &SAtlasVisualizer::OnAtlasPageChanged )
					.Content()
					[
						SNew( STextBlock )
						.Text( this, &SAtlasVisualizer::OnGetSelectedItemText )
					]
				]
				+ SHorizontalBox::Slot()
				.VAlign( VAlign_Center )
				.AutoWidth()
				.Padding(0.0,2.0f,2.0f,2.0f)
				[
					SNew( STextBlock )
					.Text( this, &SAtlasVisualizer::GetViewportSizeText )
				]
				+ SHorizontalBox::Slot()
				.Padding(20.0f,2.0f)
				.AutoWidth()
				.VAlign( VAlign_Center )
				[
					SNew( SCheckBox )
					.Visibility( this, &SAtlasVisualizer::OnGetDisplayCheckerboardVisibility )
					.OnCheckStateChanged( this, &SAtlasVisualizer::OnDisplayCheckerboardStateChanged )
					.IsChecked( this, &SAtlasVisualizer::OnGetCheckerboardState )
					.Content()
					[
						SNew( STextBlock )
						.Text( LOCTEXT("DisplayCheckerboardCheckboxLabel", "Display Checkerboard") )
					]
				]
				+ SHorizontalBox::Slot()
				.Padding(2.0f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew( STextBlock )
					.Text( this, &SAtlasVisualizer::GetZoomLevelPercentText )
				]

				+ SHorizontalBox::Slot()
				.Padding(2.0f)
				.AutoWidth()
				.VAlign( VAlign_Center )
				[
					SNew( SCheckBox )
					.OnCheckStateChanged( this, &SAtlasVisualizer::OnFitToWindowStateChanged )
					.IsChecked( this, &SAtlasVisualizer::OnGetFitToWindowState )
					.Content()
					[
						SNew( STextBlock )
						.Text( LOCTEXT("FitToWindow", "Fit to Window") )
					]
				]

				+ SHorizontalBox::Slot()
				.Padding(2.0f)
				.AutoWidth()
				.VAlign( VAlign_Center )
				[
					SNew( SButton )
					.Text( LOCTEXT("ActualSize", "Actual Size") )
					.OnClicked( this, &SAtlasVisualizer::OnActualSizeClicked )
				]
			]
			+ SVerticalBox::Slot()
			.Padding(2.0f)
			[
				SAssignNew( ScrollPanel, SAtlasVisualizerPanel )
				[
					SNew( SOverlay )
					+ SOverlay::Slot()
					[
						SNew( SImage )
						.Visibility( this, &SAtlasVisualizer::OnGetCheckerboardVisibility )
						.Image( FCoreStyle::Get().GetBrush("Checkerboard") )
					]
					+ SOverlay::Slot()
					[
						SAssignNew( Viewport, SViewport )
						.ViewportSize(this, &SAtlasVisualizer::GetViewportWidgetSize)
						.IgnoreTextureAlpha(false)
						.EnableBlending(true)
						.PreMultipliedAlpha(false)
					]
				]
			]
		]
	];

	Viewport->SetViewportInterface(SharedThis(this));
}

TUniquePtr<UE::Slate::FPaintScope> SAtlasVisualizer::Paint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, const int32 LayerId, const FWidgetStyle& InWidgetStyle, const bool bParentEnabled)
{
	if (CurrentHoveredSlotInfo.AtlasSlotRect.IsValid())
	{
		FSlateRect CurrentSlotRect = CurrentHoveredSlotInfo.AtlasSlotRect;

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId+1,
			AllottedGeometry.ToPaintGeometry((CurrentSlotRect.GetSize() * ScrollPanel->GetZoomLevel())+FVector2D(1, 1), FSlateLayoutTransform(FVector2D::Max((CurrentSlotRect.GetTopLeft()*ScrollPanel->GetZoomLevel())-FVector2D(1,1), FVector2D::ZeroVector))),
			HoveredSlotBorderBrush,
			ESlateDrawEffect::None,
			FLinearColor::Yellow
		);
	}
	return nullptr;
}

FText SAtlasVisualizer::GetToolTipText() const
{
	return HoveredAtlasSlotToolTip;
}

FIntPoint SAtlasVisualizer::GetSize() const
{
	if (FSlateShaderResource* TargetTexture = GetViewportRenderTargetTexture())
	{
		return FIntPoint(TargetTexture->GetWidth(), TargetTexture->GetHeight());
	}

	return FIntPoint(0, 0);
}

bool SAtlasVisualizer::RequiresVsync() const
{
	return false;
}

FSlateShaderResource* SAtlasVisualizer::GetViewportRenderTargetTexture() const
{
	if( SelectedAtlasPage < AtlasProvider->GetNumAtlasPages() )
	{
		return AtlasProvider->GetAtlasPageResource(SelectedAtlasPage);
	}

	return nullptr;
}

bool SAtlasVisualizer::IsViewportTextureAlphaOnly() const
{
	if (SelectedAtlasPage < AtlasProvider->GetNumAtlasPages())
	{
		return AtlasProvider->IsAtlasPageResourceAlphaOnly(SelectedAtlasPage);
	}

	return false;
}

void SAtlasVisualizer::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	CurrentHoveredSlotInfo = FAtlasSlotInfo();
}

void SAtlasVisualizer::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	CurrentHoveredSlotInfo = FAtlasSlotInfo();
}

FReply SAtlasVisualizer::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
#if WITH_ATLAS_DEBUGGING
	if (!MouseEvent.GetCursorDelta().IsNearlyZero() && !HasMouseCapture() && !MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
	{
		FVector2D LocalPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()) / ScrollPanel->GetZoomLevel();

		FAtlasSlotInfo PrevSlotInfo = CurrentHoveredSlotInfo;

		CurrentHoveredSlotInfo = AtlasProvider->GetAtlasSlotInfoAtPosition(LocalPos.IntPoint(), SelectedAtlasPage);

		if (CurrentHoveredSlotInfo != PrevSlotInfo)
		{
			RebuildToolTip(CurrentHoveredSlotInfo);
		}

		return FReply::Handled();
	}
#endif

	return FReply::Unhandled();
}

void SAtlasVisualizer::RebuildToolTip(const FAtlasSlotInfo& Info)
{
	if(Info.TextureName != NAME_None)
	{
		FTextBuilder Builder;
		Builder.AppendLine(FText::FromName(Info.TextureName));

		TArray<FName> Resources = FSlateStyleRegistry::GetSylesUsingBrush(Info.TextureName);

		if (!Resources.IsEmpty())
		{
			Builder.AppendLine(LOCTEXT("AtlasDebuggingToolTipTitle", "\nUsed by:"));
			for (FName Name : Resources)
			{
				Builder.AppendLine(Name);
			}
		}

		SetToolTipText(Builder.ToText());
	}
	else
	{
		SetToolTipText(FText::GetEmpty());
	}
}

FText SAtlasVisualizer::GetViewportSizeText() const
{
	const FIntPoint ViewportSize = GetSize();
	return FText::Format(LOCTEXT("PageSizeXY", "({0} x {1})"), ViewportSize.X, ViewportSize.Y);
}

FVector2D SAtlasVisualizer::GetViewportWidgetSize() const
{
	const FIntPoint ViewportSize = GetSize();
	return FVector2D((float)ViewportSize.X, (float)ViewportSize.Y);
}

FText SAtlasVisualizer::GetZoomLevelPercentText() const
{
	if( ScrollPanel.IsValid() )
	{
		return FText::Format( LOCTEXT("ZoomLevelPercent", "Zoom Level: {0}"), FText::AsPercent(ScrollPanel->GetZoomLevel()) );
	}

	return FText::GetEmpty();
}

void SAtlasVisualizer::OnFitToWindowStateChanged( ECheckBoxState NewState )
{
	if( ScrollPanel.IsValid() )
	{
		if( NewState == ECheckBoxState::Checked )
		{
			ScrollPanel->FitToWindow();
		}
		else
		{
			ScrollPanel->FitToSize();
		}
	}
}

ECheckBoxState SAtlasVisualizer::OnGetFitToWindowState() const
{
	if( ScrollPanel.IsValid() )
	{
		return (ScrollPanel->IsFitToWindow()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Undetermined;
}

FReply SAtlasVisualizer::OnActualSizeClicked()
{
	if( ScrollPanel.IsValid() )
	{
		ScrollPanel->FitToSize();
	}

	return FReply::Handled();
}

EVisibility SAtlasVisualizer::OnGetDisplayCheckerboardVisibility() const
{
	return IsViewportTextureAlphaOnly() ? EVisibility::Collapsed : EVisibility::Visible;
}

void SAtlasVisualizer::OnDisplayCheckerboardStateChanged( ECheckBoxState NewState )
{
	bDisplayCheckerboard = NewState == ECheckBoxState::Checked;
}

ECheckBoxState SAtlasVisualizer::OnGetCheckerboardState() const
{
	return bDisplayCheckerboard ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

EVisibility SAtlasVisualizer::OnGetCheckerboardVisibility() const
{
	return bDisplayCheckerboard && !IsViewportTextureAlphaOnly() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SAtlasVisualizer::OnComboOpening()
{
	const int32 NumAtlasPages = AtlasProvider->GetNumAtlasPages();

	AtlasPages.Empty();
	for( int32 AtlasIndex = 0; AtlasIndex < NumAtlasPages; ++AtlasIndex )
	{
		AtlasPages.Add( MakeShareable( new int32( AtlasIndex ) ) );
	}

	TSharedPtr<int32> SelectedComboEntry;
	if( SelectedAtlasPage < NumAtlasPages )
	{
		SelectedComboEntry = AtlasPages[SelectedAtlasPage];
	}
	else if( AtlasPages.Num() > 0 )
	{
		SelectedAtlasPage = 0;
		SelectedComboEntry = AtlasPages[0];
	}

	AtlasPageCombo->ClearSelection();
	AtlasPageCombo->RefreshOptions();
	AtlasPageCombo->SetSelectedItem(SelectedComboEntry);
}

FText SAtlasVisualizer::OnGetSelectedItemText() const
{
	if( SelectedAtlasPage < AtlasProvider->GetNumAtlasPages() )
	{
		return FText::Format( LOCTEXT("PageX", "Page {0}"), FText::AsNumber(SelectedAtlasPage) );
	}
	
	return LOCTEXT("SelectAPage", "Select a page");
}

void SAtlasVisualizer::OnAtlasPageChanged( TSharedPtr<int32> AtlasPage, ESelectInfo::Type SelectionType )
{
	if( AtlasPage.IsValid() )
	{
		SelectedAtlasPage = *AtlasPage;
	}
}

TSharedRef<SWidget> SAtlasVisualizer::OnGenerateWidgetForCombo( TSharedPtr<int32> AtlasPage )
{
	return 
	SNew( STextBlock )
	.Text( FText::Format( LOCTEXT("PageX", "Page {0}"), FText::AsNumber(*AtlasPage) ) );
}


#undef LOCTEXT_NAMESPACE
