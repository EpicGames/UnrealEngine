// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Rendering/RenderingCommon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"

class ISlateAtlasProvider;
class SAtlasVisualizerPanel;

class SAtlasVisualizer : public ISlateViewport, public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SAtlasVisualizer )
		: _AtlasProvider()
		{}

		SLATE_ARGUMENT( ISlateAtlasProvider*, AtlasProvider )

	SLATE_END_ARGS()

	// ISlateViewport
	virtual FIntPoint GetSize() const override;
	virtual bool RequiresVsync() const override;
	virtual FSlateShaderResource* GetViewportRenderTargetTexture() const override;
	virtual bool IsViewportTextureAlphaOnly() const override;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent);
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual TUniquePtr<UE::Slate::FPaintScope> Paint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, const int32 LayerId, const FWidgetStyle& InWidgetStyle, const bool bParentEnabled) override;
	virtual FText GetToolTipText() const;

	void Construct(const FArguments& InArgs);
private:
	void RebuildToolTip(const FAtlasSlotInfo& Info);
	FText GetViewportSizeText() const;
	FVector2D GetViewportWidgetSize() const;
	FText GetZoomLevelPercentText() const;
	void OnFitToWindowStateChanged( ECheckBoxState NewState );
	ECheckBoxState OnGetFitToWindowState() const;
	FReply OnActualSizeClicked();
	EVisibility OnGetDisplayCheckerboardVisibility() const;
	void OnDisplayCheckerboardStateChanged( ECheckBoxState NewState );
	ECheckBoxState OnGetCheckerboardState() const;
	EVisibility OnGetCheckerboardVisibility() const;
	void OnComboOpening();
	FText OnGetSelectedItemText() const;
	void OnAtlasPageChanged( TSharedPtr<int32> AtlasPage, ESelectInfo::Type SelectionType );
	TSharedRef<SWidget> OnGenerateWidgetForCombo( TSharedPtr<int32> AtlasPage );
private:
	FText HoveredAtlasSlotToolTip;
	ISlateAtlasProvider* AtlasProvider;
	TSharedPtr<SComboBox<TSharedPtr<int32>>> AtlasPageCombo;
	TArray<TSharedPtr<int32>> AtlasPages;
	TSharedPtr<SAtlasVisualizerPanel> ScrollPanel;
	int32 SelectedAtlasPage;
	bool bDisplayCheckerboard;

	const FSlateBrush* HoveredSlotBorderBrush;
	FAtlasSlotInfo CurrentHoveredSlotInfo;
};
