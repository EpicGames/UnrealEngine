// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNavigationSimulationList.h"
#include "Models/NavigationSimulationNode.h"

#include "Models/WidgetReflectorNode.h"

#include "Styling/WidgetReflectorStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/NavigationSimulationOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SNullWidget.h"


#define LOCTEXT_NAMESPACE "SSlateNavigationSimulation"


//***********************************************************
//SNavigationSimulationListInternal
namespace SNavigationSimulationListInternal
{
	static FName HeaderRow_Widget = TEXT("Widget");
	static FName HeaderRow_Navigation_Left = TEXT("NavigationLeft");
	static FName HeaderRow_Navigation_Right = TEXT("NavigationRight");
	static FName HeaderRow_Navigation_Up = TEXT("NavigationUp");
	static FName HeaderRow_Navigation_Down = TEXT("NavigationDown");
	static FName HeaderRow_Navigation_Previous = TEXT("NavigationPrevious");
	static FName HeaderRow_Navigation_Next = TEXT("NavigationNext");

//***********************************************************
//SListElement
	class SListElement : public SMultiColumnTableRow<FNavigationSimulationWidgetNodePtr>
	{
	public:
		SLATE_BEGIN_ARGS(SListElement) {}
			SLATE_ARGUMENT(FNavigationSimulationWidgetNodePtr, WidgetItem)
		SLATE_END_ARGS()

		void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
		{
			WidgetItem = Args._WidgetItem;

			SMultiColumnTableRow<FNavigationSimulationWidgetNodePtr>::Construct(
				FSuperRowType::FArguments()
				.Padding(1.0f),
				OwnerTableView
			);
		}

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			static const FText Text_Left = LOCTEXT("ShortNavigation_Left", "L");
			static const FText Text_Right = LOCTEXT("ShortNavigation_Right", "R");
			static const FText Text_Up = LOCTEXT("ShortNavigation_Up", "U");
			static const FText Text_Down = LOCTEXT("ShortNavigation_Down", "D");
			static const FText Text_Previous = LOCTEXT("ShortNavigation_Previous", "P");
			static const FText Text_Next = LOCTEXT("ShortNavigation_Next", "N");

			auto BuildNavigationText = [&](const TArray<FNavigationSimulationWidgetNodeItem, TInlineAllocator<4>>& List, EUINavigation Navigation, const FText& Text, const FSlateColor& Color)
					-> TSharedRef<SWidget>
				{
					const FNavigationSimulationWidgetNodeItem* SimulationItemPtr = List.FindByPredicate([Navigation](const FNavigationSimulationWidgetNodeItem& Item){ return Item.NavigationType == Navigation;});
					if (SimulationItemPtr && !SimulationItemPtr->Destination.IsWidgetExplicitlyNull())
					{
						return SNew(STextBlock)
							.Text(Text)
							.ColorAndOpacity(Color);
					}
					return SNullWidget::NullWidget;
				};

			if (ColumnName == HeaderRow_Widget)
			{
				return SNew(STextBlock)
					.Text(WidgetItem->NavigationSource.WidgetTypeAndShortName);
			}
			else if (ColumnName == HeaderRow_Navigation_Left)
			{
				return BuildNavigationText(WidgetItem->Simulations, EUINavigation::Left, Text_Left, FSlateColor(FColor::Cyan));
			}
			else if (ColumnName == HeaderRow_Navigation_Right)
			{
				return BuildNavigationText(WidgetItem->Simulations, EUINavigation::Right, Text_Right, FLinearColor::Yellow);
			}
			else if (ColumnName == HeaderRow_Navigation_Up)
			{
				return BuildNavigationText(WidgetItem->Simulations, EUINavigation::Up, Text_Up, FLinearColor::Red);
			}
			else if (ColumnName == HeaderRow_Navigation_Down)
			{
				return BuildNavigationText(WidgetItem->Simulations, EUINavigation::Down, Text_Down, FLinearColor::Blue);
			}
			else if (ColumnName == HeaderRow_Navigation_Previous)
			{
				return BuildNavigationText(WidgetItem->Simulations, EUINavigation::Previous, Text_Previous, FSlateColor());
			}
			else if (ColumnName == HeaderRow_Navigation_Next)
			{
				return BuildNavigationText(WidgetItem->Simulations, EUINavigation::Next, Text_Next, FSlateColor());
			}
			return SNullWidget::NullWidget;
		}

	private:
		FNavigationSimulationWidgetNodePtr WidgetItem;
	};

	class SDetailView : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SDetailView) {}
			SLATE_EVENT(FSimpleWidgetDelegate, OnNavigateToLiveWidget)
			SLATE_EVENT(FOnSnapshotWidgetAction, OnNavigateToSnapshotWidget)
		SLATE_END_ARGS()

		void Construct(const FArguments& Args);
		void SetWidgetItem(const FNavigationSimulationWidgetNodePtr& NewWidgetItem);
		void SetNavigation(EUINavigation NewNavigation);
		void HandleNavigateToWidgetInfo(FNavigationSimulationWidgetInfo::TPointerAsInt WidgetPtr, TWeakPtr<const SWidget> WidgetLive);

	private:
		ECheckBoxState GetNavigationSlotChecked(EUINavigation Navigation) const;
		void HandleNavigationSlotCheckStateChanged(ECheckBoxState NewState, EUINavigation Navigation);
		bool IsNavigationSlotEnabled(EUINavigation Navigation) const;

		TSharedRef<SWidget> BuildContent();
		TSharedRef<SWidget> BuildNaviationSlots();
		TSharedRef<SWidget> BuildNaviationSlot(EUINavigation Navigation);
		void BuildSlot(TSharedRef<SGridPanel> GridPanel, int32 RowIndex, const FText& Label, const FText& Tooltip, const FNavigationSimulationWidgetInfo& WidgetInfo);
		void BuildSlot(TSharedRef<SGridPanel> GridPanel, int32 RowIndex, const FText& Label, const FText& Tooltip, FSlateNavigationEventSimulator::ERoutedReason NavigationRule);
		void BuildSlot(TSharedRef<SGridPanel> GridPanel, int32 RowIndex, const FText& Label, const FText& Tooltip, EUINavigationRule NavigationRule);
		void BuildSlot(TSharedRef<SGridPanel> GridPanel, int32 RowIndex, const FText& Label, const FText& Tooltip, uint8 Flag);

	private:
		FNavigationSimulationWidgetNodePtr WidgetItem;
		EUINavigation NavigationToDisplay;
		//~ If the navigation is not available for the WidgetItem, change it to something else and come back to the previously selected navigation
		EUINavigation CurrentNavigationToDisplay;
		TSharedPtr<SScrollBox> DetailOwner;

		FSimpleWidgetDelegate OnNavigateToLiveWidget;
		FOnSnapshotWidgetAction OnNavigateToSnapshotWidget;
	};
}

//***********************************************************
//SNavigationSimulationSnapshotList
void SNavigationSimulationSnapshotList::Construct(const FArguments& Args, const FOnSnapshotWidgetAction& InOnSnapshotWidgetSelected, const FOnSnapshotWidgetAction& InOnNavigateToSnapshotWidget)
{
	OnSnapshotWidgetSelected = InOnSnapshotWidgetSelected;
	OnNavigateToSnapshotWidget = InOnNavigateToSnapshotWidget;
	SNavigationSimulationListBase::Construct(Args, ENavigationSimulationNodeType::Snapshot);
}

int32 SNavigationSimulationSnapshotList::PaintNodesWithOffset(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FVector2D& RootDrawOffset)
{
	TArray<FNavigationSimulationWidgetNodePtr> SelectedItems = GetSelectedItems();
	const TArray<FNavigationSimulationWidgetNodePtr>& ListToPaint = SelectedItems.Num() > 0 ? SelectedItems : GetItemsSource();
	return FNavigationSimulationOverlay::PaintSnapshotNode(ListToPaint, AllottedGeometry, OutDrawElements, LayerId, RootDrawOffset);
}

void SNavigationSimulationSnapshotList::HandleSourceListSelectionChangedImpl(const FNavigationSimulationWidgetNodePtr& Item) const
{
	if (Item)
	{
		OnSnapshotWidgetSelected.ExecuteIfBound(Item->NavigationSource.WidgetPtr);
	}
	else
	{
		OnSnapshotWidgetSelected.ExecuteIfBound(0);
	}
}

void SNavigationSimulationSnapshotList::HandleNavigateToWidgetImpl(TWeakPtr<const SWidget> LiveWidget, FNavigationSimulationWidgetInfo::TPointerAsInt SnapshotWidget) const
{
	if (SnapshotWidget != 0)
	{
		OnNavigateToSnapshotWidget.ExecuteIfBound(SnapshotWidget);
	}
}

//***********************************************************
//SNavigationSimulationLiveList
void SNavigationSimulationLiveList::Construct(const FArguments& Args, const FSimpleWidgetDelegate& InOnLiveWidgetSelected, const FSimpleWidgetDelegate& InOnNavigateToLiveWidget)
{
	OnLiveWidgetSelected = InOnLiveWidgetSelected;
	OnNavigateToLiveWidget = InOnNavigateToLiveWidget;
	SNavigationSimulationListBase::Construct(Args, ENavigationSimulationNodeType::Live);
}

void SNavigationSimulationLiveList::SelectWidget(const TSharedPtr<SWidget>& Widget)
{
	SelectLiveWidget(Widget);
}

int32 SNavigationSimulationLiveList::PaintSimuationResult(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId)
{
	TArray<FNavigationSimulationWidgetNodePtr> SelectedItems = GetSelectedItems();
	const TArray<FNavigationSimulationWidgetNodePtr>& ListToPaint = SelectedItems.Num() > 0 ? SelectedItems : GetItemsSource();
	return FNavigationSimulationOverlay::PaintLiveNode(ListToPaint, AllottedGeometry, OutDrawElements, LayerId);
}

void SNavigationSimulationLiveList::HandleSourceListSelectionChangedImpl(const FNavigationSimulationWidgetNodePtr& Item) const
{
	if (Item)
	{
		OnLiveWidgetSelected.ExecuteIfBound(Item->NavigationSource.WidgetLive.Pin());
	}
	else
	{
		OnLiveWidgetSelected.ExecuteIfBound(TSharedPtr<SWidget>());
	}
}

void SNavigationSimulationLiveList::HandleNavigateToWidgetImpl(TWeakPtr<const SWidget> LiveWidget, FNavigationSimulationWidgetInfo::TPointerAsInt SnapshotWidget) const
{
	if (TSharedPtr<const SWidget> Pinned = LiveWidget.Pin())
	{
		OnNavigateToLiveWidget.ExecuteIfBound(Pinned);
	}
}

//***********************************************************
//SNavigationSimulationListBase
void SNavigationSimulationListBase::Construct(const FArguments& Args, ENavigationSimulationNodeType InNodeType)
{
	NodeType = InNodeType;
	bIsInSelectionGuard = false;
	if (Args._ListItemsSource)
	{
		ItemsSource = *Args._ListItemsSource;
	}

	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(EOrientation::Orient_Vertical)
		.Style(FAppStyle::Get(), "SplitterDark")

		+ SSplitter::Slot()
		.Value(0.5f)
		[
			SAssignNew(ListView, SListView<FNavigationSimulationWidgetNodePtr>)
			.ListItemsSource(&ItemsSource)
			.SelectionMode(ESelectionMode::SingleToggle)
			.OnGenerateRow(this, &SNavigationSimulationListBase::MakeSourceListViewWidget)
			.OnSelectionChanged(this, &SNavigationSimulationListBase::HandleSourceListSelectionChanged)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+ SHeaderRow::Column(SNavigationSimulationListInternal::HeaderRow_Widget)
				.FillWidth(.75f)
				.DefaultLabel(FText::GetEmpty())
				+ SHeaderRow::Column(SNavigationSimulationListInternal::HeaderRow_Navigation_Left)
				.FixedWidth(15.f)
				.DefaultLabel(FText::GetEmpty())
				+ SHeaderRow::Column(SNavigationSimulationListInternal::HeaderRow_Navigation_Right)
				.FixedWidth(15.f)
				.DefaultLabel(FText::GetEmpty())
				+ SHeaderRow::Column(SNavigationSimulationListInternal::HeaderRow_Navigation_Up)
				.FixedWidth(15.f)
				.DefaultLabel(FText::GetEmpty())
				+ SHeaderRow::Column(SNavigationSimulationListInternal::HeaderRow_Navigation_Down)
				.FixedWidth(15.f)
				.DefaultLabel(FText::GetEmpty())
				+ SHeaderRow::Column(SNavigationSimulationListInternal::HeaderRow_Navigation_Previous)
				.FixedWidth(15.f)
				.DefaultLabel(FText::GetEmpty())
				+ SHeaderRow::Column(SNavigationSimulationListInternal::HeaderRow_Navigation_Next)
				.FixedWidth(15.f)
				.DefaultLabel(FText::GetEmpty())
			)
		]

		+ SSplitter::Slot()
		.Value(0.5f)
		[
			SAssignNew(ElementDetail, SNavigationSimulationListInternal::SDetailView)
			.OnNavigateToLiveWidget(this, &SNavigationSimulationListBase::HandleNavigateToLiveWidget)
			.OnNavigateToSnapshotWidget(this, &SNavigationSimulationListBase::HandleNavigateToSnapshotWidget)
		]
	];
}

void SNavigationSimulationListBase::SetSimulationResult(TArray<FSlateNavigationEventSimulator::FSimulationResult> SimulationResult)
{
	ItemsSource = FNavigationSimulationNodeUtils::BuildNavigationSimulationNodeListForLive(SimulationResult);
	ListView->RebuildList();
	ElementDetail->SetWidgetItem(FNavigationSimulationWidgetNodePtr());
}

void SNavigationSimulationListBase::SetListItemsSource(const TArray<FNavigationSimulationWidgetNodePtr>& InItemsSource)
{
	ItemsSource = InItemsSource;
	ListView->RebuildList();
	ElementDetail->SetWidgetItem(FNavigationSimulationWidgetNodePtr());
}

void SNavigationSimulationListBase::SelectLiveWidget(const TSharedPtr<const SWidget>& Widget)
{
	if (bIsInSelectionGuard)
	{
		return;
	}

	const int32 FoundIndex = FNavigationSimulationNodeUtils::IndexOfLiveWidget(ItemsSource, Widget);
	if (FoundIndex != INDEX_NONE)
	{
		ListView->SetSelection(ItemsSource[FoundIndex]);
	}
	else
	{
		ListView->ClearSelection();
	}
}

void SNavigationSimulationListBase::SelectSnapshotWidget(FNavigationSimulationWidgetInfo::TPointerAsInt SnapshotWidget)
{
	if (bIsInSelectionGuard)
	{
		return;
	}

	const int32 FoundIndex = FNavigationSimulationNodeUtils::IndexOfSnapshotWidget(ItemsSource, SnapshotWidget);
	if (FoundIndex != INDEX_NONE)
	{
		ListView->SetSelection(ItemsSource[FoundIndex]);
	}
	else
	{
		ListView->ClearSelection();
	}
}

TArray<FNavigationSimulationWidgetNodePtr> SNavigationSimulationListBase::GetSelectedItems() const
{
	TArray<FNavigationSimulationWidgetNodePtr> Result;
	ListView->GetSelectedItems(Result);
	return MoveTemp(Result);
}

TSharedRef<ITableRow> SNavigationSimulationListBase::MakeSourceListViewWidget(FNavigationSimulationWidgetNodePtr Item, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SNavigationSimulationListInternal::SListElement, OwnerTable)
	.WidgetItem(Item);
}

void SNavigationSimulationListBase::HandleSourceListSelectionChanged(FNavigationSimulationWidgetNodePtr Item, ESelectInfo::Type SelectionType)
{
	if (bIsInSelectionGuard)
	{
		return;
	}

	ElementDetail->SetWidgetItem(Item);
	if (SelectionType != ESelectInfo::Direct)
	{
		TGuardValue<bool> Tmp(bIsInSelectionGuard, true);
		HandleSourceListSelectionChangedImpl(Item);
	}
}

void SNavigationSimulationListBase::HandleNavigateToLiveWidget(TWeakPtr<const SWidget> LiveWidget)
{
	if (bIsInSelectionGuard)
	{
		return;
	}

	TGuardValue<bool> Tmp(bIsInSelectionGuard, true);
	HandleNavigateToWidgetImpl(LiveWidget, 0);
}

void SNavigationSimulationListBase::HandleNavigateToSnapshotWidget(FNavigationSimulationWidgetInfo::TPointerAsInt SnapshotWidget)
{
	if (bIsInSelectionGuard)
	{
		return;
	}

	TGuardValue<bool> Tmp(bIsInSelectionGuard, true);
	HandleNavigateToWidgetImpl(TWeakPtr<const SWidget>(), SnapshotWidget);
}

namespace SNavigationSimulationListInternal
{
//***********************************************************
//SDetailView
void SDetailView::Construct(const FArguments& Args)
{
	OnNavigateToLiveWidget = Args._OnNavigateToLiveWidget;
	OnNavigateToSnapshotWidget = Args._OnNavigateToSnapshotWidget;

	NavigationToDisplay = EUINavigation::Down;
	CurrentNavigationToDisplay = EUINavigation::Down;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 4.f)
		[
			SNew(SBorder)
			.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			[
				BuildNaviationSlots()
			]
		]

		+ SVerticalBox::Slot()
		.Padding(0.f, 4.f)
		[
			SNew(SBorder)
			.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			[
				SAssignNew(DetailOwner, SScrollBox)
				.Orientation(Orient_Vertical)

				+ SScrollBox::Slot()
				[
					BuildContent()
				]
			]
		]
	];
}

void SDetailView::SetWidgetItem(const FNavigationSimulationWidgetNodePtr& NewWidgetItem)
{
	if (WidgetItem != NewWidgetItem)
	{
		WidgetItem = NewWidgetItem;

		if (WidgetItem == nullptr || WidgetItem->Simulations.Num() == 0)
		{
			CurrentNavigationToDisplay = EUINavigation::Invalid;
		}
		else
		{
			EUINavigation LookForNavigation = NavigationToDisplay;
			const FNavigationSimulationWidgetNodeItem* FoundItem = WidgetItem->Simulations.FindByPredicate([LookForNavigation](const FNavigationSimulationWidgetNodeItem& SimulationItem) { return SimulationItem.NavigationType == LookForNavigation; });
			if (FoundItem)
			{
				CurrentNavigationToDisplay = NavigationToDisplay;
			}
			else
			{
				CurrentNavigationToDisplay = EUINavigation::Invalid;
				for (const FNavigationSimulationWidgetNodeItem& SimulationItem :WidgetItem->Simulations)
				{
					if (!SimulationItem.Destination.IsWidgetExplicitlyNull())
					{
						CurrentNavigationToDisplay = SimulationItem.NavigationType;
					}
				}
			}
		}
		DetailOwner->ClearChildren();
		DetailOwner->AddSlot()
		[
			BuildContent()
		];
	}
}

void SDetailView::SetNavigation(EUINavigation NewNavigation)
{
	NavigationToDisplay = NewNavigation;
	if (CurrentNavigationToDisplay != NavigationToDisplay)
	{
		CurrentNavigationToDisplay = NavigationToDisplay;
		DetailOwner->ClearChildren();
		DetailOwner->AddSlot()
		[
			BuildContent()
		];
	}
}

void SDetailView::HandleNavigateToWidgetInfo(FNavigationSimulationWidgetInfo::TPointerAsInt WidgetSnaphot, TWeakPtr<const SWidget> WidgetLive)
{
	if (WidgetItem)
	{
		if (WidgetItem->NodeType == ENavigationSimulationNodeType::Live)
		{
			OnNavigateToLiveWidget.ExecuteIfBound(WidgetLive);
		}
		else
		{
			OnNavigateToSnapshotWidget.ExecuteIfBound(WidgetSnaphot);
		}
	}
}

ECheckBoxState SDetailView::GetNavigationSlotChecked(EUINavigation Navigation) const
{
	return (Navigation == CurrentNavigationToDisplay) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SDetailView::HandleNavigationSlotCheckStateChanged(ECheckBoxState NewState, EUINavigation Navigation)
{
	if (NewState == ECheckBoxState::Checked)
	{
		SetNavigation(Navigation);
	}
}

bool SDetailView::IsNavigationSlotEnabled(EUINavigation Navigation) const
{
	return WidgetItem && WidgetItem->Simulations.ContainsByPredicate([Navigation](const FNavigationSimulationWidgetNodeItem& SimulationItem) { return SimulationItem.NavigationType == Navigation; });
}

TSharedRef<SWidget> SDetailView::BuildNaviationSlots()
{
	TSharedRef<SUniformGridPanel> WrapBox = SNew(SUniformGridPanel);

	for (int32 Index = 0; Index < static_cast<int32>(EUINavigation::Num); ++Index)
	{
		WrapBox->AddSlot(Index, 0)
		[
			BuildNaviationSlot(static_cast<EUINavigation>(Index))
		];
	}

	return WrapBox;
}

TSharedRef<SWidget> SDetailView::BuildNaviationSlot(EUINavigation Navigation)
{
	return SNew(SCheckBox)
	.ForegroundColor(FSlateColor::UseForeground())
	.IsChecked(this, &SDetailView::GetNavigationSlotChecked, Navigation)
	.OnCheckStateChanged(this, &SDetailView::HandleNavigationSlotCheckStateChanged, Navigation)
	.IsEnabled(this, &SDetailView::IsNavigationSlotEnabled, Navigation)
	[
		SNew(SBox)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.Padding(FMargin(4.0, 2.0))
		[
			SNew(STextBlock)
			.Text(UEnum::GetDisplayValueAsText(Navigation))
		]
	];
}

TSharedRef<SWidget> SDetailView::BuildContent()
{
	TSharedRef<SWidget> NewContent = SNullWidget::NullWidget;
	if (WidgetItem)
	{
		for(const FNavigationSimulationWidgetNodeItem& SimulationItem : WidgetItem->Simulations)
		{
			if (SimulationItem.NavigationType == CurrentNavigationToDisplay)
			{
				TSharedRef<SGridPanel> GridPanel = SAssignNew(NewContent, SGridPanel)
					.FillColumn(0, 0.5f)
					.FillColumn(1, 0.5f);

				int32 RowIndex = 0;
				BuildSlot(
					GridPanel,
					RowIndex++,
					LOCTEXT("NavigationSourceLabel", "Source"),
					FText::GetEmpty(),
					WidgetItem->NavigationSource);
				BuildSlot(
					GridPanel,
					RowIndex++,
					LOCTEXT("NavigationDestinationLabel", "Destination"),
					FText::GetEmpty(),
					SimulationItem.Destination);
				GridPanel->AddSlot(0, RowIndex++)
					.ColumnSpan(2)
					[
						SNew(SSeparator)
					];
				BuildSlot(
					GridPanel,
					RowIndex++,
					LOCTEXT("RoutedReasonLabel", "Routed"),
					FText::GetEmpty(),
					SimulationItem.RoutedReason);
				BuildSlot(
					GridPanel,
					RowIndex++,
					LOCTEXT("ReplyEventHandlerLabel", "Reply Event Handler"),
					FText::GetEmpty(),
					SimulationItem.ReplyEventHandler);
				BuildSlot(
					GridPanel,
					RowIndex++,
					LOCTEXT("ReplyBoundaryRuleLabel", "Reply Boundary Rule"),
					FText::GetEmpty(),
					SimulationItem.ReplyBoundaryRule);
				BuildSlot(
					GridPanel,
					RowIndex++,
					LOCTEXT("ReplyFocusRecipientLabel", "Reply Focus Recipient (Custom)"),
					FText::GetEmpty(),
					SimulationItem.ReplyFocusRecipient);
				BuildSlot(
					GridPanel,
					RowIndex++,
					LOCTEXT("WidgetThatShouldReceivedFocusLabel", "Should Focus"),
					LOCTEXT("WidgetThatShouldReceivedFocusTooltip", "Widget that should received the focus after the navigation."),
					SimulationItem.WidgetThatShouldReceivedFocus);
				BuildSlot(
					GridPanel,
					RowIndex++,
					LOCTEXT("NavigationFocusedLabel", "Focused"),
					FText::GetEmpty(),
					SimulationItem.FocusedWidget);
				BuildSlot(
					GridPanel,
					RowIndex++,
					LOCTEXT("IsDynamicLabel", "Is Dynamic"),
					FText::GetEmpty(),
					SimulationItem.bIsDynamic);
				BuildSlot(
					GridPanel,
					RowIndex++,
					LOCTEXT("AlwaysHandleNavigationAttemptLabel", "Always Handle Navigation"),
					FText::GetEmpty(),
					SimulationItem.bAlwaysHandleNavigationAttempt);
				BuildSlot(
					GridPanel,
					RowIndex++,
					LOCTEXT("CanFindWidgetForSetFocusLabel", "Can Find Widget for Set Focus"),
					FText::GetEmpty(),
					SimulationItem.bCanFindWidgetForSetFocus);
				BuildSlot(
					GridPanel,
					RowIndex++,
					LOCTEXT("RoutedHandlerHasNavigationMetaLabel", "Reply Event Handler has Meta Data"),
					FText::GetEmpty(),
					SimulationItem.bRoutedHandlerHasNavigationMeta);

				break;
			}
		}
	}
	return NewContent;
}

void SDetailView::BuildSlot(TSharedRef<SGridPanel> GridPanel, int32 RowIndex, const FText& Label, const FText& Tooltip, const FNavigationSimulationWidgetInfo& WidgetInfo)
{
	GridPanel->AddSlot(0, RowIndex)
	[
		SNew(STextBlock)
		.Text(Label)
		.ToolTipText(Tooltip)
	];

	TSharedPtr<SWidget> WidgetInfoWidget;
	if (WidgetInfo.IsWidgetExplicitlyNull())
	{
		WidgetInfoWidget = SNew(STextBlock)
		.Text(WidgetInfo.WidgetTypeAndShortName);
	}
	else
	{
		WidgetInfoWidget = SNew(SHyperlink)
			.Text(WidgetInfo.WidgetTypeAndShortName)
			.OnNavigate(this, &SDetailView::HandleNavigateToWidgetInfo, WidgetInfo.WidgetPtr, WidgetInfo.WidgetLive);
	}

	GridPanel->AddSlot(1, RowIndex)
	.HAlign(HAlign_Right)
	[
		WidgetInfoWidget.ToSharedRef()
	];
}

void SDetailView::BuildSlot(TSharedRef<SGridPanel> GridPanel, int32 RowIndex, const FText& Label, const FText& Tooltip, FSlateNavigationEventSimulator::ERoutedReason NavigationRule)
{
	GridPanel->AddSlot(0, RowIndex)
	[
		SNew(STextBlock)
		.Text(Label)
		.ToolTipText(Tooltip)
	];

	GridPanel->AddSlot(1, RowIndex)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Text(FSlateNavigationEventSimulator::ToText(NavigationRule))
	];
}

void SDetailView::BuildSlot(TSharedRef<SGridPanel> GridPanel, int32 RowIndex, const FText& Label, const FText& Tooltip, EUINavigationRule NavigationRule)
{
	GridPanel->AddSlot(0, RowIndex)
	[
		SNew(STextBlock)
		.Text(Label)
		.ToolTipText(Tooltip)
	];

	GridPanel->AddSlot(1, RowIndex)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Text(UEnum::GetDisplayValueAsText(NavigationRule))
	];
}

void SDetailView::BuildSlot(TSharedRef<SGridPanel> GridPanel, int32 RowIndex, const FText& Label, const FText& Tooltip, uint8 Flag)
{
	GridPanel->AddSlot(0, RowIndex)
	[
		SNew(STextBlock)
		.Text(Label)
		.ToolTipText(Tooltip)
	];

	GridPanel->AddSlot(1, RowIndex)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Text(Flag ? LOCTEXT("TrueValue", "True") : LOCTEXT("FalseValue", "False"))
	];
}
}//SNavigationSimulationListInternal

#undef LOCTEXT_NAMESPACE
