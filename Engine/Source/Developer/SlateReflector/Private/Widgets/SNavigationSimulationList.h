// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "INavigationEventSimulationView.h"
#include "Models/NavigationSimulationNode.h"
#include "SlateNavigationEventSimulator.h"
#include "ISlateReflectorModule.h"

template <typename ItemType> class SListView;
class ITableRow;
class SGridPanel;
class SNavigationSimulationListBase;
class SScrollBox;
class STableViewBase;
class SWrapBox;
class SWidget;

namespace SNavigationSimulationListInternal
{
	class SDetailView;
}


DECLARE_DELEGATE_OneParam(FOnSnapshotWidgetAction, FNavigationSimulationWidgetInfo::TPointerAsInt);

class SNavigationSimulationListBase : public INavigationEventSimulationView
{
public:
	SLATE_BEGIN_ARGS(SNavigationSimulationListBase)
		: _ListItemsSource(nullptr)
		{}
		SLATE_ARGUMENT(const TArray<FNavigationSimulationWidgetNodePtr>*, ListItemsSource)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, ENavigationSimulationNodeType NodeType);

	//~ Begin INavigationEventSimulationView interface
	virtual void SetSimulationResult(TArray<FSlateNavigationEventSimulator::FSimulationResult> SimulationResult) override;
	//~ End INavigationEventSimulationView interface

	void SetListItemsSource(const TArray<FNavigationSimulationWidgetNodePtr>& ListItems);
	void SelectLiveWidget(const TSharedPtr<const SWidget>& Widget);
	void SelectSnapshotWidget(FNavigationSimulationWidgetInfo::TPointerAsInt SnapshotWidget);

protected:
	virtual void HandleSourceListSelectionChangedImpl(const FNavigationSimulationWidgetNodePtr& Item) const = 0;
	virtual void HandleNavigateToWidgetImpl(TWeakPtr<const SWidget> LiveWidget, FNavigationSimulationWidgetInfo::TPointerAsInt SnapshotWidget) const = 0;

	TArray<FNavigationSimulationWidgetNodePtr> GetSelectedItems() const;
	const TArray<FNavigationSimulationWidgetNodePtr>& GetItemsSource() const { return ItemsSource; };

private:
	TSharedRef<ITableRow> MakeSourceListViewWidget(FNavigationSimulationWidgetNodePtr Item, const TSharedRef<STableViewBase>& OwnerTable) const;
	void HandleSourceListSelectionChanged(FNavigationSimulationWidgetNodePtr Item, ESelectInfo::Type SelectionType);

	void HandleNavigateToLiveWidget(TWeakPtr<const SWidget> LiveWidget);
	void HandleNavigateToSnapshotWidget(FNavigationSimulationWidgetInfo::TPointerAsInt SnapshotWidget);

private:
	ENavigationSimulationNodeType NodeType;
	TSharedPtr<SListView<FNavigationSimulationWidgetNodePtr>> ListView;
	TSharedPtr<SNavigationSimulationListInternal::SDetailView> ElementDetail;
	TArray<FNavigationSimulationWidgetNodePtr> ItemsSource;

protected:
	bool bIsInSelectionGuard;
};

class SNavigationSimulationSnapshotList : public SNavigationSimulationListBase
{
public:
	void Construct(const FArguments& Args, const FOnSnapshotWidgetAction& OnSnapshotWidgetSelected, const FOnSnapshotWidgetAction& OnNavigateToSnapshotWidget);

	//~ Begin INavigationEventSimulationView interface
	virtual void SelectWidget(const TSharedPtr<SWidget>& Widget) override {}
	virtual int32 PaintSimuationResult(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) override { return LayerId; }
	//~ End INavigationEventSimulationView interface

	int32 PaintNodesWithOffset(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FVector2D& RootDrawOffset);

protected:
	//~ Begin SNavigationSimulationListBase interface
	virtual void HandleSourceListSelectionChangedImpl(const FNavigationSimulationWidgetNodePtr& Item) const override;
	virtual void HandleNavigateToWidgetImpl(TWeakPtr<const SWidget> LiveWidget, FNavigationSimulationWidgetInfo::TPointerAsInt SnapshotWidget) const override;
	//~ End SNavigationSimulationListBase interface

private:
	FOnSnapshotWidgetAction OnSnapshotWidgetSelected;
	FOnSnapshotWidgetAction OnNavigateToSnapshotWidget;
};

class SNavigationSimulationLiveList : public SNavigationSimulationListBase
{
public:
	void Construct(const FArguments& Args, const FSimpleWidgetDelegate& OnLiveWidgetSelected, const FSimpleWidgetDelegate& OnNavigateToLiveWidget);

	//~ Begin INavigationEventSimulationView interface
	virtual void SelectWidget(const TSharedPtr<SWidget>& Widget) override;
	virtual int32 PaintSimuationResult(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) override;
	//~ End INavigationEventSimulationView interface

protected:
	//~ Begin SNavigationSimulationListBase interface
	virtual void HandleSourceListSelectionChangedImpl(const FNavigationSimulationWidgetNodePtr& Item) const override;
	virtual void HandleNavigateToWidgetImpl(TWeakPtr<const SWidget> LiveWidget, FNavigationSimulationWidgetInfo::TPointerAsInt SnapshotWidget) const override;
	//~ End SNavigationSimulationListBase interface

private:
	FSimpleWidgetDelegate OnLiveWidgetSelected;
	FSimpleWidgetDelegate OnNavigateToLiveWidget;
};