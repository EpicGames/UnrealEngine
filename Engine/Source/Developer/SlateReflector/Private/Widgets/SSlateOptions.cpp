// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SSlateOptions.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/IConsoleManager.h"
#include "SlateGlobals.h"
#include "SlateReflectorModule.h"
#include "Styling/CoreStyle.h"
#include "Styling/WidgetReflectorStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SInvalidationPanel.h"
#include "Widgets/SWidgetReflector.h"

#define LOCTEXT_NAMESPACE "SSlateOptions"

void SSlateOptions::Construct( const FArguments& InArgs )
{
	struct Local
	{
		static void AddMenuEntry(FMenuBuilder& MenuBuilder, const FSlateIcon& Icon, const FText& Label, const TCHAR* ConsoleVariable, bool bCanEdit = true)
		{
			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(ConsoleVariable);
			if (CVar)
			{
				FTextBuilder TooltipText;
				TooltipText.AppendLine(FString(CVar->GetHelp()));
				TooltipText.AppendLine(FString(ConsoleVariable));

				MenuBuilder.AddMenuEntry(
					Label,
					TooltipText.ToText(),
					Icon,
					FUIAction(
						FExecuteAction::CreateLambda([CVar]()
						{
							// If the variable was changed in the command line, we might not be able to change it anymore from the UI so we remove the flag
							CVar->ClearFlags(EConsoleVariableFlags::ECVF_SetByCommandline);
							CVar->Set(!CVar->GetBool(), EConsoleVariableFlags::ECVF_SetByCode);
						}),
						FCanExecuteAction::CreateLambda([bCanEdit](){ return bCanEdit; }),
						FGetActionCheckState::CreateLambda([CVar]() { return CVar->GetBool() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton);
			}
		}
		static void AddSubMenuEntry(FMenuBuilder& MenuBuilder, const FSlateIcon& Icon, const FText& Label, const TCHAR* ConsoleVariable, const FNewMenuDelegate& InSubMenu, bool bCanEdit = true)
		{
			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(ConsoleVariable);
			if (CVar)
			{
				FTextBuilder TooltipText;
				TooltipText.AppendLine(FString(CVar->GetHelp()));
				TooltipText.AppendLine(FString(ConsoleVariable));

				MenuBuilder.AddSubMenu(
					Label,
					TooltipText.ToText(),
					InSubMenu,
					FUIAction(
					FExecuteAction::CreateLambda([CVar]() {
							// If the variable was changed in the command line, we might not be able to change it anymore from the UI so we remove the flag
							CVar->ClearFlags(EConsoleVariableFlags::ECVF_SetByCommandline);
							CVar->Set(!CVar->GetBool(), EConsoleVariableFlags::ECVF_SetByCode);
						}),
						FCanExecuteAction::CreateLambda([bCanEdit](){ return bCanEdit; }),
						FGetActionCheckState::CreateLambda([CVar]() { return CVar->GetBool() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton,
					false,
					Icon
				);
			}
		}
	};

	struct GlobalLocal : Local
	{
		static TSharedRef<SWidget> FillToolbar()
		{
			const bool bShouldCloseWindowAfterMenuSelection = true;
			FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

			FSlateIcon Icon(FWidgetReflectorStyle::GetStyleSetName(), "Icon.Empty");

			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("EnableFastWidgetPath", "Fast Widget Path"), TEXT("Slate.EnableFastWidgetPath"), false);
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("EnableToolTips", "Enable Tooltips"), TEXT("Slate.EnableTooltips"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("GlobalInvalidation", "Global Invalidation"), TEXT("Slate.EnableGlobalInvalidation"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("DisabledEffect", "Transparent Disabled Effect"), TEXT("Slate.ApplyDisabledEffectOnWidgets"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("FocusOnPick", "Focus On Pick"), TEXT("Slate.EnableFocusOnPick"));

			return MenuBuilder.MakeWidget();
		}
	};

	struct DebugLocal : Local
	{
		static void FillPaintSubmenu(FMenuBuilder& MenuBuilder)
		{
			FSlateIcon Icon(FWidgetReflectorStyle::GetStyleSetName(), "Icon.Empty");
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("Paint_OnlyProjectContent", "Only Project Content"), TEXT("SlateDebugger.Paint.OnlyProjectContent"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("Paint_OnlyInvalidationRoot", "Only Invalidation Root"), TEXT("SlateDebugger.Paint.OnlyInvalidationRoot"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("Paint_ToggleWidgetNameList", "Toggle Widget Name List"), TEXT("SlateDebugger.Paint.EnableWidgetNameList"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("Paint_Draw Border", "Draw Border"), TEXT("SlateDebugger.Paint.DrawBorder"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("Paint_Draw Fill", "Draw Fill"), TEXT("SlateDebugger.Paint.DrawFill"));
		}
		static void FillPrepassSubmenu(FMenuBuilder& MenuBuilder)
		{
			FSlateIcon Icon(FWidgetReflectorStyle::GetStyleSetName(), "Icon.Empty");
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("Prepass_OnlyProjectContent", "Only Project Content"), TEXT("SlateDebugger.Prepass.OnlyProjectContent"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("Prepass_OnlyInvalidationRoot", "Only Invalidation Root"), TEXT("SlateDebugger.Prepass.OnlyInvalidationRoot"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("Prepass_ToggleWidgetNameList", "Toggle Widget Name List"), TEXT("SlateDebugger.Prepass.EnableWidgetNameList"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("Prepass_Draw Border", "Draw Border"), TEXT("SlateDebugger.Prepass.DrawBorder"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("Prepass_Draw Fill", "Draw Fill"), TEXT("SlateDebugger.Prepass.DrawFill"));
		}
		
		static TSharedRef<SWidget> FillToolbar()
		{
			const bool bShouldCloseWindowAfterMenuSelection = true;
			FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

			FSlateIcon Icon(FWidgetReflectorStyle::GetStyleSetName(), "Icon.Empty");
#if WITH_SLATE_DEBUGGING
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("EnableInvalidationPanels", "Enable InvalidationBox"), TEXT("Slate.EnableInvalidationPanels"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("InvalidationDebugging", "Show Invalidation"), TEXT("SlateDebugger.Invalidate.Enable"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("InvalidationRootDebugging", "Show Root Invalidation"), TEXT("SlateDebugger.InvalidationRoot.Enable"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("UpdateDebugging", "Show Update"), TEXT("SlateDebugger.Update.Enable"));
			AddSubMenuEntry(MenuBuilder, Icon, LOCTEXT("PaintDebugging", "Show Paint"), TEXT("SlateDebugger.Paint.Enable"), FNewMenuDelegate::CreateStatic(&FillPaintSubmenu));
			AddSubMenuEntry(MenuBuilder, Icon, LOCTEXT("PrepassDebugging", "Show Prepass"), TEXT("SlateDebugger.Prepass.Enable"), FNewMenuDelegate::CreateStatic(&FillPrepassSubmenu));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("ShowClipping", "Show Clipping"), TEXT("Slate.ShowClipping"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("DebugCulling", "Debug Culling"), TEXT("Slate.DebugCulling"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("ShowHitTestGrid", "Show HitTestGrid"), TEXT("Slate.HitTestGridDebugging"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("DesignerRetainedRendering", "Designer Retained Rendering"), TEXT("Slate.EnableDesignerRetainedRendering"));
#endif // WITH_SLATE_DEBUGGING
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("ShowBatching", "Show Batching"), TEXT("Slate.ShowBatching"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("ShowOverdraw", "Show Overdraw"), TEXT("Slate.ShowOverdraw"));
			
			return MenuBuilder.MakeWidget();
		}
	};

	struct ValidationLocal : Local
	{
		static TSharedRef<SWidget> FillToolbar()
		{
			const bool bShouldCloseWindowAfterMenuSelection = true;
			FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

			FSlateIcon Icon(FWidgetReflectorStyle::GetStyleSetName(), "Icon.Empty");
#if WITH_SLATE_DEBUGGING
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("VerifyParentChildrenRelationship", "Verify Parent/Children Relationship"), TEXT("Slate.VerifyParentChildrenRelationship"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("VerifyLayerId", "Verify LayerId"), TEXT("Slate.VerifyWidgetLayerId"));
			AddMenuEntry(MenuBuilder, Icon, LOCTEXT("VerifyOutgoingLayerId", "Verify Outgoing LayerId"), TEXT("Slate.EnsureOutgoingLayerId"));
#endif // WITH_SLATE_DEBUGGING

			{
				MenuBuilder.BeginSection("InvalidationRoot", LOCTEXT("Invalidation", "Invalidation"));
	#if WITH_SLATE_DEBUGGING
				AddMenuEntry(MenuBuilder, Icon, LOCTEXT("EnsureAllVisibleWidgetsPaint", "Verify Visible Widgets Paint"), TEXT("Slate.EnsureAllVisibleWidgetsPaint"));
	#endif // WITH_SLATE_DEBUGGING
	#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
				AddMenuEntry(MenuBuilder, Icon, LOCTEXT("VerifyWidgetList", "Verify Widget List"), TEXT("Slate.InvalidationRoot.VerifyWidgetList"));
				AddMenuEntry(MenuBuilder, Icon, LOCTEXT("VerifyWidgetIndex", "Verify Widget's index"), TEXT("Slate.InvalidationRoot.VerifyWidgetsIndex"));
				AddMenuEntry(MenuBuilder, Icon, LOCTEXT("VerifyWidgetPtr", "Verify Widget Pointer"), TEXT("Slate.InvalidationRoot.VerifyValidWidgets"));
				AddMenuEntry(MenuBuilder, Icon, LOCTEXT("VerifyHittestGrid", "Verify Hittest Grid"), TEXT("Slate.InvalidationRoot.VerifyHittestGrid"));
				AddMenuEntry(MenuBuilder, Icon, LOCTEXT("VerifyCachedElement", "Verify Cached Elements"), TEXT("Slate.InvalidationRoot.VerifyCachedElementDataList"));
				AddMenuEntry(MenuBuilder, Icon, LOCTEXT("VefiryVisibility", "Verify Visibility"), TEXT("Slate.InvalidationRoot.VerifyWidgetVisibility"));
				AddMenuEntry(MenuBuilder, Icon, LOCTEXT("VerifyVolatility", "Verify Volatility"), TEXT("Slate.InvalidationRoot.VerifyWidgetVolatile"));
				AddMenuEntry(MenuBuilder, Icon, LOCTEXT("VerifyUpdateList", "Verify Update List"), TEXT("Slate.InvalidationRoot.VerifyWidgetUpdateList"));
				AddMenuEntry(MenuBuilder, Icon, LOCTEXT("VerifyUpdateOnce", "Verify Updated Once"), TEXT("Slate.InvalidationRoot.VerifyWidgetsAreUpdatedOnce"));
				AddMenuEntry(MenuBuilder, Icon, LOCTEXT("VerifySlateAttributes", "Verify Attributes"), TEXT("Slate.InvalidationRoot.VerifySlateAttribute"));
	#endif // UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
				MenuBuilder.EndSection();
			}			

			return MenuBuilder.MakeWidget();
		}
	};


	FToolBarBuilder ToolbarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);
	ToolbarBuilder.SetStyle(&FWidgetReflectorStyle::Get(), "BoldSlimToolbar");

	ToolbarBuilder.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateStatic(&GlobalLocal::FillToolbar),
		LOCTEXT("FlagLabel", "Flags"),
		FText::GetEmpty(),
		FSlateIcon(),
		false
	);

	ToolbarBuilder.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateStatic(&DebugLocal::FillToolbar),
		LOCTEXT("DebugLabel", "Debug Options"),
		FText::GetEmpty(),
		FSlateIcon(),
		false
	);

	ToolbarBuilder.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateStatic(&ValidationLocal::FillToolbar),
		LOCTEXT("EnsureLabel", "Runtime Validation"),
		FText::GetEmpty(),
		FSlateIcon(),
		false
	);
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		.Padding(2.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.f, 0.f, 4.f, 0.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AppScale", "Application Scale: "))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(100.f)
				.MaxDesiredWidth(250.f)
				[
					SNew(SSpinBox<float>)
					.Value(this, &SSlateOptions::HandleAppScaleSliderValue)
					.MinValue(0.50f)
					.MaxValue(3.0f)
					.Delta(0.01f)
					.OnValueChanged(this, &SSlateOptions::HandleAppScaleSliderChanged)
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.Padding(FMargin(5.0f, 0.0f))
			[
				ToolbarBuilder.MakeWidget()
			]
		]
	];
}

void SSlateOptions::HandleAppScaleSliderChanged(float NewValue)
{
	FSlateApplication::Get().SetApplicationScale(NewValue);
}

float SSlateOptions::HandleAppScaleSliderValue() const
{
	return FSlateApplication::Get().GetApplicationScale();
}

#undef LOCTEXT_NAMESPACE
