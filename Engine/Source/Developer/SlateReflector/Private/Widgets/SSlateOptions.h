// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

/**
 * Widget that show debug options
 */
class SSlateOptions : public SCompoundWidget
{
public:
	SLATE_USER_ARGS(SSlateOptions) { }
	SLATE_END_ARGS()

	virtual void Construct(const FArguments& InArgs);

private:
	void HandleAppScaleSliderChanged(float NewValue);
	float HandleAppScaleSliderValue() const;
};

