// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SVisualElementsTab.h"

#include "Framework/Application/SlateApplication.h"
#include "Models/WidgetReflectorNode.h"
#include "VisualTreeCapture.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "SVisualElementsTab"

/* FVisualElementEntry
 *****************************************************************************/

UE::SlateReflector::Private::FVisualElementEntry::FVisualElementEntry(const FVisualEntry& InEntry)
	: Widget(InEntry.Widget)
	, LayerId(InEntry.LayerId)
	, ElementType(InEntry.ElementType)
{
}

/* Column names
 *****************************************************************************/

namespace VisualElementsTabColumns
{
	static const FName Widget = "Widget";
	static const FName Type   = "Type";
	static const FName Layer  = "Layer";
}

/* Helpers
 *****************************************************************************/

static FText GetElementTypeName(EElementType Type)
{
	switch (Type)
	{
	case EElementType::ET_Box:             return LOCTEXT("ET_Box",           "Box");
	case EElementType::ET_DebugQuad:       return LOCTEXT("ET_DebugQuad",     "DebugQuad");
	case EElementType::ET_Text:            return LOCTEXT("ET_Text",           "Text");
	case EElementType::ET_ShapedText:      return LOCTEXT("ET_ShapedText",     "ShapedText");
	case EElementType::ET_Spline:          return LOCTEXT("ET_Spline",         "Spline");
	case EElementType::ET_Line:            return LOCTEXT("ET_Line",           "Line");
	case EElementType::ET_Gradient:        return LOCTEXT("ET_Gradient",       "Gradient");
	case EElementType::ET_Viewport:        return LOCTEXT("ET_Viewport",       "Viewport");
	case EElementType::ET_Border:          return LOCTEXT("ET_Border",         "Border");
	case EElementType::ET_Custom:          return LOCTEXT("ET_Custom",         "Custom");
	case EElementType::ET_CustomVerts:     return LOCTEXT("ET_CustomVerts",    "CustomVerts");
	case EElementType::ET_PostProcessPass: return LOCTEXT("ET_PostProcess",    "PostProcess");
	case EElementType::ET_RoundedBox:      return LOCTEXT("ET_RoundedBox",     "RoundedBox");
	default:                               return LOCTEXT("ET_Unknown",        "Unknown");
	}
}

/* SVisualEntryTableRow
 *****************************************************************************/

using FVisualElementEntry = UE::SlateReflector::Private::FVisualElementEntry;

class SVisualEntryTableRow : public SMultiColumnTableRow<TSharedPtr<FVisualElementEntry>>
{
public:
	SLATE_BEGIN_ARGS(SVisualEntryTableRow) {}
		SLATE_EVENT(SVisualElementsTab::FOnWidgetSelected, OnWidgetSelected)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FVisualElementEntry> InEntry, const TSharedRef<STableViewBase>& OwnerTable)
	{
		Entry = InEntry;
		OnWidgetSelected = InArgs._OnWidgetSelected;
		FSuperRowType::Construct(FSuperRowType::FArguments(), OwnerTable);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == VisualElementsTabColumns::Widget)
		{
			FText WidgetName = LOCTEXT("InvalidWidget", "(invalid)");
			if (TSharedPtr<const SWidget> Widget = Entry->Widget.Pin())
			{
				WidgetName = FWidgetReflectorNodeUtils::GetWidgetTypeAndShortName(Widget);
			}

			return SNew(SBox)
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.0f, 0.0f))
				[
					SNew(SHyperlink)
					.Text(WidgetName)
					.OnNavigate(this, &SVisualEntryTableRow::HandleNavigate)
				];
		}

		if (ColumnName == VisualElementsTabColumns::Type)
		{
			return SNew(SBox)
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.0f, 0.0f))
				[
					SNew(STextBlock)
					.Text(GetElementTypeName(Entry->ElementType))
				];
		}

		if (ColumnName == VisualElementsTabColumns::Layer)
		{
			return SNew(SBox)
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.0f, 0.0f))
				[
					SNew(STextBlock)
					.Text(FText::AsNumber(Entry->LayerId))
				];
		}

		return SNullWidget::NullWidget;
	}

private:
	void HandleNavigate()
	{
		OnWidgetSelected.ExecuteIfBound(Entry->Widget.Pin());
	}

	TSharedPtr<FVisualElementEntry> Entry;
	SVisualElementsTab::FOnWidgetSelected OnWidgetSelected;
};

/* Shared header row builder
 *****************************************************************************/

static TSharedRef<SHeaderRow> MakeHeaderRow()
{
	return SNew(SHeaderRow)
		+ SHeaderRow::Column(VisualElementsTabColumns::Widget)
		.DefaultLabel(LOCTEXT("ColumnWidget", "Widget"))
		.FillWidth(1.0f)
		+ SHeaderRow::Column(VisualElementsTabColumns::Type)
		.DefaultLabel(LOCTEXT("ColumnType", "Type"))
		.FixedWidth(80.0f)
		+ SHeaderRow::Column(VisualElementsTabColumns::Layer)
		.DefaultLabel(LOCTEXT("ColumnLayer", "Layer"))
		.FixedWidth(50.0f);
}

/* SVisualElementsTab
 *****************************************************************************/

void SVisualElementsTab::Construct(const FArguments& InArgs, TNotNull<const FVisualTreeCapture*> InVisualCapture)
{
	OnWidgetSelected = InArgs._OnWidgetSelected;
	VisualCapture = InVisualCapture;
	FilterWidgets = InArgs._FilterWidgets;
	IsPickingActive = InArgs._IsPickingActive;

	ChildSlot
	[
		SNew(SVerticalBox)

		// Toolbar
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(2.0f, 2.0f))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.IsChecked(this, &SVisualElementsTab::GetFilterByWidgetChecked)
				.OnCheckStateChanged(this, &SVisualElementsTab::HandleFilterByWidgetChanged)
				.ToolTipText(LOCTEXT("FilterTooltip", "When checked, the main list only shows elements produced by the selected widgets in the tree."))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FilterLabel", "Filter by Selected Widgets"))
				]
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SSplitter)
			.Orientation(Orient_Vertical)

			// Main entry list
			+ SSplitter::Slot()
			.Value(0.6f)
			[
				SAssignNew(ListView, SListView<TSharedPtr<FVisualElementEntry>>)
				.ListItemsSource(&ListEntries)
				.OnGenerateRow(this, &SVisualElementsTab::HandleGenerateRow)
				.HeaderRow(MakeHeaderRow())
			]

			// Region entry list
			+ SSplitter::Slot()
			.Value(0.4f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.Padding(FMargin(4.0f, 2.0f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("RegionHeader", "Region (entries inside cursor element)"))
					]
				]

				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SAssignNew(RegionListView, SListView<TSharedPtr<FVisualElementEntry>>)
					.ListItemsSource(&RegionEntries)
					.OnGenerateRow(this, &SVisualElementsTab::HandleGenerateRegionRow)
					.HeaderRow(MakeHeaderRow())
				]
			]
		]
	];
}

void SVisualElementsTab::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	check(VisualCapture);

	const bool bHadElements = ListEntries.Num() > 0 || RegionEntries.Num() > 0;
	const bool bIsPickingActive = IsPickingActive.Get(false);
	ListEntries.Reset();
	if (bIsPickingActive || !VisualCapture->HasElements())
	{
		RegionEntries.Reset();
	}

	const TArray<TSharedPtr<const SWidget>> PinnedFilters = FilterWidgets.Get();
	const FVector2f ScreenCursorPos = FSlateApplication::Get().GetCursorPos();

	TArray<TSharedRef<SWindow>> AllWindows;
	FSlateApplication::Get().GetAllVisibleWindowsOrdered(AllWindows);

	for (const TSharedRef<SWindow>& Window : AllWindows)
	{
		TSharedPtr<const FVisualTreeSnapshot> Snapshot = VisualCapture->GetVisualTreeForWindow(&Window.Get());
		if (!Snapshot)
		{
			continue;
		}

		// Populate the main list (all windows).
		for (const FVisualEntry& SourceEntry : Snapshot->Entries)
		{
			if (bFilterByWidget && !PinnedFilters.Contains(SourceEntry.Widget.Pin()))
			{
				continue;
			}
			ListEntries.Add(MakeShared<FVisualElementEntry>(SourceEntry));
		}

		// Populate the region list only while picking is active and for the window that contains the cursor.
		// Use GetRectInScreen() to check containment in screen space, then convert
		// the cursor to window-local space before passing to PickAll.
		if (bIsPickingActive && RegionEntries.IsEmpty())
		{
			const FSlateRect WindowRect = Window->GetRectInScreen();
			const bool bCursorInWindow = ScreenCursorPos.X >= WindowRect.Left
				&& ScreenCursorPos.X <  WindowRect.Right
				&& ScreenCursorPos.Y >= WindowRect.Top
				&& ScreenCursorPos.Y <  WindowRect.Bottom;

			if (bCursorInWindow)
			{
				// Convert to window-local space: subtract the window's screen position.
				// GetLocalToWindowTransform() is local->window and contains only DPI scale.
				const FVector2f WindowLocalCursor = ScreenCursorPos - FVector2f(Window->GetPositionInScreen());

				for (int32 Index : Snapshot->PickAll(WindowLocalCursor))
				{
					RegionEntries.Add(MakeShared<FVisualElementEntry>(Snapshot->Entries[Index]));
				}
			}
		}
	}

	const bool bHasElements = ListEntries.Num() > 0 || RegionEntries.Num() > 0;
	if (bHadElements || bHasElements)
	{
		ListView->RequestListRefresh();
		RegionListView->RequestListRefresh();
	}
}

ECheckBoxState SVisualElementsTab::GetFilterByWidgetChecked() const
{
	return bFilterByWidget ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SVisualElementsTab::HandleFilterByWidgetChanged(ECheckBoxState NewState)
{
	bFilterByWidget = (NewState == ECheckBoxState::Checked);
}

TSharedRef<ITableRow> SVisualElementsTab::HandleGenerateRow(TSharedPtr<FVisualElementEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SVisualEntryTableRow, Entry, OwnerTable)
		.OnWidgetSelected(OnWidgetSelected);
}

TSharedRef<ITableRow> SVisualElementsTab::HandleGenerateRegionRow(TSharedPtr<FVisualElementEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SVisualEntryTableRow, Entry, OwnerTable)
		.OnWidgetSelected(OnWidgetSelected);
}

#undef LOCTEXT_NAMESPACE
