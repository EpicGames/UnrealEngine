// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Rendering/DrawElementCoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class SWidget;
class FVisualEntry;
class FVisualTreeCapture;
class FVisualTreeSnapshot;
class ITableRow;
class STableViewBase;

namespace UE::SlateReflector::Private
{

/** Lightweight display entry for the Elements tab: widget pointer, layer, and element type. */
struct FVisualElementEntry
{
	TWeakPtr<const SWidget> Widget;
	int32 LayerId = INDEX_NONE;
	EElementType ElementType = EElementType::ET_Count;

	explicit FVisualElementEntry(const FVisualEntry& InEntry);
};

} // namespace UE::SlateReflector::Private

/**
 * Tab displaying all FVisualEntry items captured by FVisualTreeCapture.
 */
class SVisualElementsTab : public SCompoundWidget
{
private:
	using Super = SCompoundWidget;

	using FVisualElementEntry = UE::SlateReflector::Private::FVisualElementEntry;

public:
	DECLARE_DELEGATE_OneParam(FOnWidgetSelected, TSharedPtr<const SWidget>)

	SLATE_BEGIN_ARGS(SVisualElementsTab) {}
		SLATE_EVENT(FOnWidgetSelected, OnWidgetSelected)
		SLATE_ATTRIBUTE(TArray<TSharedPtr<const SWidget>>, FilterWidgets)
		SLATE_ATTRIBUTE(bool, IsPickingActive)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TNotNull<const FVisualTreeCapture*> VisualCapture);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	ECheckBoxState GetFilterByWidgetChecked() const;
	void HandleFilterByWidgetChanged(ECheckBoxState NewState);

	TSharedRef<ITableRow> HandleGenerateRow(TSharedPtr<FVisualElementEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedRef<ITableRow> HandleGenerateRegionRow(TSharedPtr<FVisualElementEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable);

	/** Returns true if all four corners of Entry are inside the Region's quad. */
	static bool IsEntryInsideRegion(const FVisualEntry& Entry, const FVisualEntry& Region);

	FOnWidgetSelected OnWidgetSelected;
	const FVisualTreeCapture* VisualCapture = nullptr;
	bool bFilterByWidget = false;
	bool bDrawablePickingActive = false;

	TAttribute<TArray<TSharedPtr<const SWidget>>> FilterWidgets;
	TAttribute<bool> IsPickingActive;

	// Main list (filtered subset of snapshot entries).
	TSharedPtr<SListView<TSharedPtr<FVisualElementEntry>>> ListView;
	TArray<TSharedPtr<FVisualElementEntry>> ListEntries;

	// Region list (entries whose quads fit inside the entry under cursor).
	TSharedPtr<SListView<TSharedPtr<FVisualElementEntry>>> RegionListView;
	TArray<TSharedPtr<FVisualElementEntry>> RegionEntries;
};
