// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SWidgetEventLog.h"

#if WITH_SLATE_DEBUGGING

#include "Debugging/SlateDebugging.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GenericPlatform/ICursor.h"
#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"
#include "Logging/MessageLog.h"
#include "Styling/CoreStyle.h"
#include "Styling/WidgetReflectorStyle.h"
#include "Types/ReflectionMetadata.h"
#include "Types/SlateEnums.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "WidgetEventLog"

static FName NAME_WidgetEvents(TEXT("WidgetEvents"));


void SWidgetEventLog::Construct(const FArguments& InArgs, TSharedPtr<const SWidget> InReflectorWidget)
{
	ReflectorWidget = InReflectorWidget;
	OnWidgetTokenActivated = InArgs._OnWidgetTokenActivated;
	bFilterWidgetReflectorEvent = false;

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>(TEXT("MessageLog"));
	if (!MessageLogModule.IsRegisteredLogListing(NAME_WidgetEvents))
	{
		FMessageLogInitializationOptions InitOptions;
		InitOptions.bScrollToBottom = true;
		MessageLogModule.RegisterLogListing(NAME_WidgetEvents, LOCTEXT("WidgetEventLog", "Widget Events"), InitOptions);
	}
	TSharedRef<IMessageLogListing> MessageLogListing = MessageLogModule.GetLogListing(NAME_WidgetEvents);

	FSlimHorizontalToolBarBuilder ToolbarBuilderGlobal(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);

	ToolbarBuilderGlobal.BeginSection("Log");
	{
		ToolbarBuilderGlobal.AddWidget(SNew(SCheckBox)
			.ForegroundColor(FSlateColor::UseForeground())
			.IsChecked(this, &SWidgetEventLog::HandleFilterWidgetReflectorEventIsChecked)
			.OnCheckStateChanged(this, &SWidgetEventLog::HandleFilterWidgetReflectorEventStateChanged)
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(FMargin(4.0f, 2.0f))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FilterWidgetReflectorEvent", "Hide events originated from the Widget Reflector"))
				]
			]);

		FTextBuilder TooltipText;

		ToolbarBuilderGlobal.AddComboButton(
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction(),
				FGetActionCheckState()
			),
			FOnGetContent::CreateSP(this, &SWidgetEventLog::OnGenerateCategoriesMenu),
			LOCTEXT("CategoryComboButtonText", "Filters"),
			TooltipText.ToText(),
			FSlateIcon(FWidgetReflectorStyle::GetStyleSetName(), "Icon.Filter"),
			false
		);


	}
	ToolbarBuilderGlobal.EndSection();


	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.Padding(4.f)
			[
				ToolbarBuilderGlobal.MakeWidget()
			]
		]

		+ SVerticalBox::Slot()
		[
			MessageLogModule.CreateLogListingWidget(MessageLogListing)
		]
	];

	GenerateList();
	UpdateListeners();
}


SWidgetEventLog::~SWidgetEventLog()
{
	RemoveListeners();
}

void SWidgetEventLog::GenerateList()
{
	// We are casting ESlateDebuggingInputEvent::MAX to uint8. Let's make sure it's a valid value.
	auto GenerateCategory = [this] (EWidgetEventType EventType)
	{
		check(GetEnum(EventType)->GetMaxEnumValue() < (int64)TNumericLimits<uint8>::Max());
		GetBitField(EventType) = TBitArray<>(false, (int32)GetEnum(EventType)->GetMaxEnumValue());
		GetCategoryFlag(EventType) = false;
	};
	GenerateCategory(EWidgetEventType::Focus);
	GenerateCategory(EWidgetEventType::Input);
	GenerateCategory(EWidgetEventType::Navigation);
	GenerateCategory(EWidgetEventType::Cursor);
	bIsWarningEnabled = false;
	bIsCaptureStateEnabled = false;
}

TSharedRef<SWidget> SWidgetEventLog::OnGenerateCategoriesMenu()
{
	const bool CloseAfterSelection = false;
	FMenuBuilder MenuBuilder(CloseAfterSelection, NULL);

	MenuBuilder.AddSubMenu(
		LOCTEXT("FocusSubMenu", "Focus"),
		FText(),
		FNewMenuDelegate::CreateSP(this, &SWidgetEventLog::OnGenerateCategoriesSubMenu, EWidgetEventType::Focus));
	MenuBuilder.AddSubMenu(
		LOCTEXT("InputSubMenu", "Input"),
		FText(),
		FNewMenuDelegate::CreateSP(this, &SWidgetEventLog::OnGenerateCategoriesSubMenu, EWidgetEventType::Input));
	MenuBuilder.AddSubMenu(
		LOCTEXT("NavigationSubMenu", "Navigation"),
		FText(),
		FNewMenuDelegate::CreateSP(this, &SWidgetEventLog::OnGenerateCategoriesSubMenu, EWidgetEventType::Navigation));
	MenuBuilder.AddSubMenu(
		LOCTEXT("CursorSubMenu", "Cursor"),
		FText(),
		FNewMenuDelegate::CreateSP(this, &SWidgetEventLog::OnGenerateCategoriesSubMenu, EWidgetEventType::Cursor));
	MenuBuilder.AddMenuEntry(
		LOCTEXT("WarningMenu", "Warning"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SWidgetEventLog::EnabledCategory, EWidgetEventType::Warning),
			FCanExecuteAction(),
			FGetActionCheckState::CreateSP(this, &SWidgetEventLog::GetEnabledCheckState, EWidgetEventType::Warning)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("CaptureMenu", "Mouse Capture"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SWidgetEventLog::EnabledCategory, EWidgetEventType::Capture),
			FCanExecuteAction(),
			FGetActionCheckState::CreateSP(this, &SWidgetEventLog::GetEnabledCheckState, EWidgetEventType::Capture)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);

	return MenuBuilder.MakeWidget();
}

void SWidgetEventLog::OnGenerateCategoriesSubMenu(FMenuBuilder& InSubMenuBuilder, EWidgetEventType EventType)
{
	TBitArray<>& EventEnabled = GetBitField(EventType);
	const UEnum* EventEnum = GetEnum(EventType);

	InSubMenuBuilder.AddMenuEntry(
		LOCTEXT("AllSubMenu", "All"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SWidgetEventLog::EnabledAllFromSubCategory, EventType),
			FCanExecuteAction(),
			FGetActionCheckState::CreateSP(this, &SWidgetEventLog::GetEnabledAllFromSubCategoryCheckState, EventType)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);

	InSubMenuBuilder.AddMenuSeparator();

	uint8 MaxEnumValue = static_cast<uint8>(EventEnum->GetMaxEnumValue());
	for (uint8 Index = 0; Index < MaxEnumValue; ++Index)
	{
#if WITH_EDITOR
		if (!EventEnum->HasMetaData(TEXT("Hidden"), Index))
#endif
		{
			InSubMenuBuilder.AddMenuEntry(
				EventEnum->GetDisplayNameTextByIndex(Index),
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SWidgetEventLog::EnabledSubCategory, EventType, Index),
					FCanExecuteAction(),
					FGetActionCheckState::CreateSP(this, &SWidgetEventLog::GetEnabledSubCategoryCheckState, EventType, Index)
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);
		}
	}
}

void SWidgetEventLog::EnabledCategory(EWidgetEventType EventType)
{
	bool& bAllEventEnabled = GetCategoryFlag(EventType);
	bAllEventEnabled = !bAllEventEnabled;
}

ECheckBoxState SWidgetEventLog::GetEnabledCheckState(EWidgetEventType EventType) const
{
	const bool bAllEventEnabled = GetCategoryFlag(EventType);
	return bAllEventEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SWidgetEventLog::EnabledAllFromSubCategory(EWidgetEventType EventType)
{
	TBitArray<>& EventEnabled = GetBitField(EventType);
	bool& bAllEventEnabled = GetCategoryFlag(EventType);

	bAllEventEnabled = !bAllEventEnabled;
	int32 NumbOfElement = EventEnabled.Num();
	EventEnabled.Init(bAllEventEnabled, NumbOfElement);
}

ECheckBoxState SWidgetEventLog::GetEnabledAllFromSubCategoryCheckState(EWidgetEventType EventType) const
{
	const bool bAllEventEnabled = GetCategoryFlag(EventType);
	return bAllEventEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SWidgetEventLog::EnabledSubCategory(EWidgetEventType EventType, uint8 Index)
{
	TBitArray<>& EventEnabled = GetBitField(EventType);
	bool& bAllEventEnabled = GetCategoryFlag(EventType);

	EventEnabled[Index] = !EventEnabled[Index];
	if (EventEnabled[Index])
	{
		// Are they all true?
		bAllEventEnabled = !EventEnabled.Contains(false);
	}
	else
	{
		bAllEventEnabled = false;
	}
}

ECheckBoxState SWidgetEventLog::GetEnabledSubCategoryCheckState(EWidgetEventType EventType, uint8 Index) const
{
	const TBitArray<>& EventEnabled = GetBitField(EventType);
	return EventEnabled[Index] ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SWidgetEventLog::HandleFilterWidgetReflectorEventStateChanged(ECheckBoxState NewValue)
{
	bFilterWidgetReflectorEvent = NewValue == ECheckBoxState::Checked;
}

ECheckBoxState SWidgetEventLog::HandleFilterWidgetReflectorEventIsChecked() const
{
	return bFilterWidgetReflectorEvent ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SWidgetEventLog::RemoveListeners()
{
	FSlateDebugging::Warning.RemoveAll(this);
	FSlateDebugging::InputEvent.RemoveAll(this);
	FSlateDebugging::FocusEvent.RemoveAll(this);
	FSlateDebugging::AttemptNavigationEvent.RemoveAll(this);
	FSlateDebugging::MouseCaptureEvent.RemoveAll(this);
	FSlateDebugging::CursorChangedEvent.RemoveAll(this);
}

void SWidgetEventLog::UpdateListeners()
{
	RemoveListeners();

	FSlateDebugging::Warning.AddRaw(this, &SWidgetEventLog::HandleWarning);
	FSlateDebugging::InputEvent.AddRaw(this, &SWidgetEventLog::HandleInputEvent);
	FSlateDebugging::FocusEvent.AddRaw(this, &SWidgetEventLog::HandleFocusEvent);
	FSlateDebugging::AttemptNavigationEvent.AddRaw(this, &SWidgetEventLog::HandleAttemptNavigationEvent);
	FSlateDebugging::MouseCaptureEvent.AddRaw(this, &SWidgetEventLog::HandleCaptureStateChangeEvent);
	FSlateDebugging::CursorChangedEvent.AddRaw(this, &SWidgetEventLog::HandleCursorChangedEvent);
}

void SWidgetEventLog::HandleWarning(const FSlateDebuggingWarningEventArgs& EventArgs) const
{
	if (!bIsWarningEnabled)
	{
		return;
	}

	if (bFilterWidgetReflectorEvent && IsInsideWidgetReflector(EventArgs.OptionalContextWidget))
	{
		return;
	}

	FMessageLog MessageLog(NAME_WidgetEvents);
	MessageLog.SuppressLoggingToOutputLog();
	TSharedRef<FTokenizedMessage> Token = MessageLog.Warning(EventArgs.ToText());
	if (EventArgs.OptionalContextWidget)
	{
		TWeakPtr<const SWidget> OptionalContextWidget = EventArgs.OptionalContextWidget;
		Token->AddToken(FActionToken::Create(
			LOCTEXT("SelectWidget", "Widget"),
			LOCTEXT("SelectWidgetTooltip", "Click to select the widget in the visualizer"),
			FOnActionTokenExecuted::CreateSP(this, &SWidgetEventLog::SelectWidget, OptionalContextWidget)
		));
	}
}

void SWidgetEventLog::HandleInputEvent(const FSlateDebuggingInputEventArgs& EventArgs) const
{
	if (!InputEnabled[static_cast<uint8>(EventArgs.InputEventType)])
	{
		return;
	}

	if (bFilterWidgetReflectorEvent && IsInsideWidgetReflector(EventArgs.HandlerWidget))
	{
		return;
	}

	FMessageLog MessageLog(NAME_WidgetEvents);
	MessageLog.SuppressLoggingToOutputLog();
	TSharedRef<FTokenizedMessage> Token = MessageLog.Info(EventArgs.ToText());
	if (EventArgs.HandlerWidget)
	{
		TWeakPtr<const SWidget> OptionalContextWidget = EventArgs.HandlerWidget;
		Token->AddToken(FActionToken::Create(
			LOCTEXT("SelectWidget", "Widget"),
			LOCTEXT("SelectWidgetTooltip", "Click to select the widget in the visualizer"),
			FOnActionTokenExecuted::CreateSP(this, &SWidgetEventLog::SelectWidget, OptionalContextWidget)
		));
	}
}

void SWidgetEventLog::HandleFocusEvent(const FSlateDebuggingFocusEventArgs& EventArgs) const
{
	if (!FocusEnabled[static_cast<uint8>(EventArgs.FocusEventType)])
	{
		return;
	}

	if (bFilterWidgetReflectorEvent && IsInsideWidgetReflector(EventArgs.OldFocusedWidgetPath))
	{
		return;
	}
	if (bFilterWidgetReflectorEvent && IsInsideWidgetReflector(EventArgs.NewFocusedWidgetPath))
	{
		return;
	}

	FMessageLog MessageLog(NAME_WidgetEvents);
	MessageLog.SuppressLoggingToOutputLog();
	TSharedRef<FTokenizedMessage> Token = MessageLog.Info(EventArgs.ToText());
	if (EventArgs.NewFocusedWidget)
	{
		TWeakPtr<const SWidget> OptionalContextWidget = EventArgs.NewFocusedWidget;
		Token->AddToken(FActionToken::Create(
			LOCTEXT("SelectWidget", "Widget"),
			LOCTEXT("SelectWidgetTooltip", "Click to select the widget in the visualizer"),
			FOnActionTokenExecuted::CreateSP(this, &SWidgetEventLog::SelectWidget, OptionalContextWidget)
		));
	}
}

void SWidgetEventLog::HandleAttemptNavigationEvent(const FSlateDebuggingNavigationEventArgs& EventArgs) const
{
	if (!NavigationEnabled[static_cast<uint8>(EventArgs.NavigationEvent.GetNavigationType())])
	{
		return;
	}

	if (bFilterWidgetReflectorEvent && IsInsideWidgetReflector(EventArgs.DestinationWidget))
	{
		return;
	}
	if (bFilterWidgetReflectorEvent && IsInsideWidgetReflector(EventArgs.NavigationSource))
	{
		return;
	}

	FMessageLog MessageLog(NAME_WidgetEvents);
	MessageLog.SuppressLoggingToOutputLog();
	TSharedRef<FTokenizedMessage> Token = MessageLog.Info(EventArgs.ToText());
	if (EventArgs.DestinationWidget)
	{
		TWeakPtr<const SWidget> OptionalContextWidget = EventArgs.DestinationWidget;
		Token->AddToken(FActionToken::Create(
			LOCTEXT("SelectWidget", "Widget"),
			LOCTEXT("SelectWidgetTooltip", "Click to select the widget in the visualizer"),
			FOnActionTokenExecuted::CreateSP(this, &SWidgetEventLog::SelectWidget, OptionalContextWidget)
		));
	}
}

void SWidgetEventLog::HandleCaptureStateChangeEvent(const FSlateDebuggingMouseCaptureEventArgs& EventArgs) const
{
	if (!bIsCaptureStateEnabled)
	{
		return;
	}

	if (bFilterWidgetReflectorEvent && IsInsideWidgetReflector(EventArgs.CaptureWidget))
	{
		return;
	}

	FMessageLog MessageLog(NAME_WidgetEvents);
	MessageLog.SuppressLoggingToOutputLog();
	TSharedRef<FTokenizedMessage> Token = MessageLog.Info(EventArgs.ToText());
	if (EventArgs.CaptureWidget)
	{
		TWeakPtr<const SWidget> OptionalContextWidget = EventArgs.CaptureWidget;
		Token->AddToken(FActionToken::Create(
			LOCTEXT("SelectWidget", "Widget"),
			LOCTEXT("SelectWidgetTooltip", "Click to select the widget in the visualizer"),
			FOnActionTokenExecuted::CreateSP(this, &SWidgetEventLog::SelectWidget, OptionalContextWidget)
		));
	}
}

void SWidgetEventLog::HandleCursorChangedEvent(const FSlateDebuggingCursorQueryEventArgs& EventArgs) const
{
	if (!CursorEnabled[static_cast<uint8>(EventArgs.Reply.GetCursorType())])
	{
		return;
	}

	if (bFilterWidgetReflectorEvent && IsInsideWidgetReflector(EventArgs.WidgetOverridingCursor))
	{
		return;
	}

	FMessageLog MessageLog(NAME_WidgetEvents);
	MessageLog.SuppressLoggingToOutputLog();
	TSharedRef<FTokenizedMessage> Token = MessageLog.Info(EventArgs.ToText());
	if (EventArgs.WidgetOverridingCursor)
	{
		TWeakPtr<const SWidget> OptionalContextWidget = EventArgs.WidgetOverridingCursor;
		Token->AddToken(FActionToken::Create(
			LOCTEXT("SelectWidget", "Widget"),
			LOCTEXT("SelectWidgetTooltip", "Click to select the widget in the visualizer"),
			FOnActionTokenExecuted::CreateSP(this, &SWidgetEventLog::SelectWidget, OptionalContextWidget)
		));
	}
}

void SWidgetEventLog::SelectWidget(TWeakPtr<const SWidget> Widget) const
{
	if (TSharedPtr<const SWidget> PinnedWidget = Widget.Pin())
	{
		OnWidgetTokenActivated.ExecuteIfBound(PinnedWidget);
	}
#if WITH_EDITOR
	else
	{
		// Create and display a notification about the tile set being modified
		FNotificationInfo Info(LOCTEXT("InvalidWidget", "Widget has been destroyed."));
		Info.ExpireDuration = 1.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
#endif //WITH_EDITOR
}

bool SWidgetEventLog::IsInsideWidgetReflector(const SWidget* Widget) const
{
	bool bResult = false;
	if (const SWidget* ReflectorWidgetPtr = ReflectorWidget.Pin().Get())
	{
		const SWidget* WidgetWindowPtr = Widget;
		while (WidgetWindowPtr && !WidgetWindowPtr->Advanced_IsWindow())
		{
			bResult = (WidgetWindowPtr == ReflectorWidgetPtr);
			if (bResult)
			{
				break;
			}
			WidgetWindowPtr = WidgetWindowPtr->GetParentWidget().Get();
		}
	}
	return bResult;
}

bool SWidgetEventLog::IsInsideWidgetReflector(TSharedRef<const SWidget> Widget) const
{
	return IsInsideWidgetReflector(&(Widget.Get()));
}

bool SWidgetEventLog::IsInsideWidgetReflector(const TSharedPtr<const SWidget>& Widget) const
{
	return IsInsideWidgetReflector(Widget.Get());
}

bool SWidgetEventLog::IsInsideWidgetReflector(const FWidgetPath& WidgetPath) const
{
	if (TSharedPtr<const SWidget> PinnedReflectorWidget = ReflectorWidget.Pin())
	{
		return WidgetPath.ContainsWidget(PinnedReflectorWidget.Get());
	}
	return false;
}

bool SWidgetEventLog::IsInsideWidgetReflector(const FWeakWidgetPath& WidgetPath) const
{
	if (TSharedPtr<const SWidget> PinnedReflectorWidget = ReflectorWidget.Pin())
	{
		return WidgetPath.ContainsWidget(PinnedReflectorWidget.Get());
	}
	return false;
}

const TBitArray<>& SWidgetEventLog::GetBitField(EWidgetEventType EventType) const
{
	switch (EventType)
	{
	case EWidgetEventType::Focus: return FocusEnabled;
	case EWidgetEventType::Input: return InputEnabled;
	case EWidgetEventType::Navigation: return NavigationEnabled;
	case EWidgetEventType::Cursor: return CursorEnabled;
	}
	check(false);
	return FocusEnabled;
}

TBitArray<>& SWidgetEventLog::GetBitField(EWidgetEventType EventType)
{
	return const_cast<TBitArray<>&>(const_cast<const SWidgetEventLog*>(this)->GetBitField(EventType));
}

const bool& SWidgetEventLog::GetCategoryFlag(EWidgetEventType EventType) const
{
	switch (EventType)
	{
	case EWidgetEventType::Warning: return bIsWarningEnabled;
	case EWidgetEventType::Focus: return bIsAllFocusEnabled;
	case EWidgetEventType::Input: return bIsAllInputEnabled;
	case EWidgetEventType::Navigation: return bIsAllNavigationEnabled;
	case EWidgetEventType::Capture: return bIsCaptureStateEnabled;
	case EWidgetEventType::Cursor: return bIsAllCursorEnabled;
	}
	check(false);
	return bIsAllFocusEnabled;
}

bool& SWidgetEventLog::GetCategoryFlag(EWidgetEventType EventType)
{
	return const_cast<bool&>(const_cast<const SWidgetEventLog*>(this)->GetCategoryFlag(EventType));
}

const UEnum* SWidgetEventLog::GetEnum(EWidgetEventType EventType) const
{
	switch (EventType)
	{
	case EWidgetEventType::Focus: return StaticEnum<ESlateDebuggingFocusEvent>();
	case EWidgetEventType::Input: return StaticEnum<ESlateDebuggingInputEvent>();
	case EWidgetEventType::Navigation: return StaticEnum<EUINavigation>();
	case EWidgetEventType::Cursor: return StaticEnum<EMouseCursor::Type>();
	}
	check(false);
	return nullptr;
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_SLATE_DEBUGGING
