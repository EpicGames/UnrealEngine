// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Debugging/SlateDebugging.h"

#if WITH_SLATE_DEBUGGING

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Layout/WidgetPath.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"

enum class ECheckBoxState : uint8;
class FMenuBuilder;

/**
 * 
 */
class SWidgetEventLog : public SCompoundWidget
{
public:
	enum class EWidgetEventType
	{
		Warning,
		Focus,
		Input,
		Navigation,
		Capture,
		Cursor,
	};

	DECLARE_DELEGATE_OneParam(FOnWidgetTokenActivated, TSharedPtr<const SWidget>);
	SLATE_BEGIN_ARGS(SWidgetEventLog)
		{}
		SLATE_EVENT(FOnWidgetTokenActivated, OnWidgetTokenActivated)
	SLATE_END_ARGS()

	virtual ~SWidgetEventLog();

	void Construct(const FArguments& InArgs, TSharedPtr<const SWidget> InReflectorWidget);

private:
	void GenerateList();

	TSharedRef<SWidget> OnGenerateCategoriesMenu();
	void OnGenerateCategoriesSubMenu(FMenuBuilder& InSubMenuBuilder, EWidgetEventType EventType);
	void EnabledCategory(EWidgetEventType EventType);
	ECheckBoxState GetEnabledCheckState(EWidgetEventType EventType) const;
	void EnabledAllFromSubCategory(EWidgetEventType EventType);
	ECheckBoxState GetEnabledAllFromSubCategoryCheckState(EWidgetEventType EventType) const;
	void EnabledSubCategory(EWidgetEventType EventType, uint8 Index);
	ECheckBoxState GetEnabledSubCategoryCheckState(EWidgetEventType EventType, uint8 Index) const;
	void HandleFilterWidgetReflectorEventStateChanged(ECheckBoxState NewValue);
	ECheckBoxState HandleFilterWidgetReflectorEventIsChecked() const;

	void RemoveListeners();
	void UpdateListeners();

	void HandleWarning(const FSlateDebuggingWarningEventArgs& EventArgs) const;
	void HandleInputEvent(const FSlateDebuggingInputEventArgs& EventArgs) const;
	void HandleFocusEvent(const FSlateDebuggingFocusEventArgs& EventArgs) const;
	void HandleAttemptNavigationEvent(const FSlateDebuggingNavigationEventArgs& EventArgs) const;
	void HandleCaptureStateChangeEvent(const FSlateDebuggingMouseCaptureEventArgs& EventArgs) const;
	void HandleCursorChangedEvent(const FSlateDebuggingCursorQueryEventArgs& EventArgs) const;
	void SelectWidget(TWeakPtr<const SWidget> Widget) const;

	bool IsInsideWidgetReflector(const SWidget* Widget) const;
	bool IsInsideWidgetReflector(TSharedRef<const SWidget> Widget) const;
	bool IsInsideWidgetReflector(const TSharedPtr<const SWidget>& Widget) const;
	bool IsInsideWidgetReflector(const FWidgetPath& WidgetPath) const;
	bool IsInsideWidgetReflector(const FWeakWidgetPath& WidgetPath) const;

	const TBitArray<>& GetBitField(EWidgetEventType EventType) const;
	TBitArray<>& GetBitField(EWidgetEventType EventType);
	const bool& GetCategoryFlag(EWidgetEventType EventType) const;
	bool& GetCategoryFlag(EWidgetEventType EventType);
	const UEnum* GetEnum(EWidgetEventType EventType) const;

	TWeakPtr<const SWidget> ReflectorWidget;
	FOnWidgetTokenActivated OnWidgetTokenActivated;

	TBitArray<> FocusEnabled;
	TBitArray<> InputEnabled;
	TBitArray<> NavigationEnabled;
	TBitArray<> CursorEnabled;
	bool bIsAllFocusEnabled;
	bool bIsAllInputEnabled;
	bool bIsAllNavigationEnabled;
	bool bIsAllCursorEnabled;

	bool bIsWarningEnabled;
	bool bIsCaptureStateEnabled;

	bool bFilterWidgetReflectorEvent;
};

#endif // WITH_SLATE_DEBUGGING