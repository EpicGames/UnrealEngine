// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SWidgetHittestGrid.h"

#if WITH_SLATE_DEBUGGING

#include "Debugging/SlateDebugging.h"
#include "Misc/ConfigCacheIni.h"
#include "Models/WidgetReflectorNode.h"
#include "SlateGlobals.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/CoreStyle.h"
#include "Styling/WidgetReflectorStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWindow.h"

#if WITH_EDITOR
#include "Styling/AppStyle.h"
#endif

#define LOCTEXT_NAMESPACE "SWidgetHittestGrid"

namespace WidgetHittestGridInternal
{
	static FName Header_Index = "index";
	static FName Header_WidgetName = "widgetname";
	static FName Header_Result = "result";
	static FName Header_WidgetInfo = "widgetinfo";

	struct FIntermediateResultNode
	{
		TWeakPtr<const SWidget> Widget;
		int32 Index = INDEX_NONE;
		FText Result;
		FText GetWidgetType() const { return FWidgetReflectorNodeUtils::GetWidgetType(Widget.Pin()); }
		FText GetWidgetTypeAndShortName() const { return FWidgetReflectorNodeUtils::GetWidgetTypeAndShortName(Widget.Pin()); }
		FText GetWidgetReadableLocation() const { return FWidgetReflectorNodeUtils::GetWidgetReadableLocation(Widget.Pin()); }
	};

	class SIntermediateResultNode : public SMultiColumnTableRow<TSharedRef<FIntermediateResultNode>>
	{
		SLATE_BEGIN_ARGS(SIntermediateResultNode){}
			SLATE_ARGUMENT(TSharedPtr<FIntermediateResultNode>, NodeToVisualize)
		SLATE_END_ARGS()
	public:
		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
		{
			NodeToVisualize = InArgs._NodeToVisualize;
			SMultiColumnTableRow<TSharedRef<FIntermediateResultNode>>::Construct(SMultiColumnTableRow<TSharedRef<FIntermediateResultNode>>::FArguments().Padding(0.f), InOwnerTableView);
		}

	public:
		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (ColumnName == Header_Index)
			{
				if (NodeToVisualize->Index >= 0)
				{
					return SNew(SBox)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::AsNumber(NodeToVisualize->Index))
					];
				}
				return SNullWidget::NullWidget;
			}
			if (ColumnName == Header_WidgetName)
			{
				return SNew(STextBlock)
				.Text(NodeToVisualize->GetWidgetTypeAndShortName());
			}
			if (ColumnName == Header_Result)
			{
				return SNew(SBox)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(NodeToVisualize->Result)
				];
			}
			if (ColumnName == Header_WidgetInfo)
			{
				return SNew(SBox)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(FMargin(2.0f, 0.0f))
				[
					SNew(STextBlock)
					.Text(NodeToVisualize->GetWidgetReadableLocation())
				];
			}
			return SNullWidget::NullWidget;
		}

	private:
		TSharedPtr<FIntermediateResultNode> NodeToVisualize;
	};
}

void SWidgetHittestGrid::Construct(const FArguments& InArgs, TSharedPtr<const SWidget> InReflectorWidget)
{
	OnWidgetSelected = InArgs._OnWidgetSelected;
	OnVisualizeWidget = InArgs._OnVisualizeWidget;
	ReflectorWidget = InReflectorWidget;
	DisplayGridFlags = FHittestGrid::EDisplayGridFlags::None;
	bDisplayAllWindows = false;
	bVisualizeOnNavigation = false;
	bRejectWidgetReflectorEvent = false;

	LoadSettings();

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.Padding(4.f)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DisplayLabel", "Display: "))
					.ColorAndOpacity(FLinearColor::White)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.Padding(4.f, 0.f, 0.f, 0.f)
				[
					SNew(SComboButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.ContentPadding(FMargin(6.0f, 2.0f))
					.IsEnabled_Lambda([](){return !GSlateHitTestGridDebugging;})
					.OnGetMenuContent(this, &SWidgetHittestGrid::GetDisplayMenuContent)
					.ButtonContent()
					[
						SNew(SBox)
						.Padding(FMargin(4.f, 0.f, 4.f, 0.f))
						[
							SNew(STextBlock)
							.Text(this, &SWidgetHittestGrid::GetDisplayButtonText)
							.ColorAndOpacity(FLinearColor::White)
						]
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.Padding(4.f, 0.f, 0.f, 0.f)
				[
					SNew(SComboButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.ContentPadding(FMargin(6.0f, 2.0f))
					.IsEnabled_Lambda([]() { return !GSlateHitTestGridDebugging; })
					.OnGetMenuContent(this, &SWidgetHittestGrid::GetFlagsMenuContent)
					.ButtonContent()
					[
						SNew(SBox)
						.Padding(FMargin(4.f, 0.f, 4.f, 0.f))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("DisplayFlagsLabel", "Display Flags"))
							.ColorAndOpacity(FLinearColor::White)
						]
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8.f, 0.f, 0.f, 0.f)
				[
					SNew(SCheckBox)
					.ForegroundColor(FSlateColor::UseForeground())
					.ToolTipText(LOCTEXT("PickOnNavigateTooltip", "Attempt to pick the widget in the Widget Reflector when navigation via the hit test grid occurred."))
					.IsChecked(this, &SWidgetHittestGrid::HandleGetVisualizeOnNavigationChecked)
					.OnCheckStateChanged(this, &SWidgetHittestGrid::HandleVisualizeOnNavigationChanged)
					[
						SNew(SBox)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.Padding(FMargin(4.0, 2.0))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PickOnNavigate", "Visualize on Navigation"))
						]
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8.f, 0.f, 0.f, 0.f)
				[
					SNew(SCheckBox)
					.ForegroundColor(FSlateColor::UseForeground())
					.ToolTipText(LOCTEXT("RejectWidgetReflectorTooltip", "Reject navigation event originated from the Widget Reflector."))
					.IsChecked(this, &SWidgetHittestGrid::HandleGetRejectWidgetReflectorChecked)
					.OnCheckStateChanged(this, &SWidgetHittestGrid::HandleRejectWidgetReflectorChanged)
					[
						SNew(SBox)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.Padding(FMargin(4.0, 2.0))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("RejectWidgetReflector", "Reject Widget Reflector navigation events"))
						]
					]
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.Padding(4.f)
			[
				ConstructNavigationDetail()
			]
		]

		+ SVerticalBox::Slot()
		[
			SAssignNew(NavigationIntermediateResultTree, SWidgetHittestGridTree)
			.ListItemsSource(&IntermediateResultNodesRoot)
			.SelectionMode(ESelectionMode::Single)
			.OnGenerateRow(this, &SWidgetHittestGrid::HandleWidgetHittestGridGenerateRow)
			.OnSelectionChanged(this, &SWidgetHittestGrid::HandleWidgetHittestGridSelectionChanged)
			.HeaderRow
			(
				SNew(SHeaderRow)

				+ SHeaderRow::Column(WidgetHittestGridInternal::Header_Index)
				.DefaultLabel(LOCTEXT("IndexHeader", "Index"))
				.FixedWidth(40.f)

				+ SHeaderRow::Column(WidgetHittestGridInternal::Header_WidgetName)
				.DefaultLabel(LOCTEXT("WidgetNameHeader", "Widget Name"))

				+ SHeaderRow::Column(WidgetHittestGridInternal::Header_Result)
				.DefaultLabel(LOCTEXT("StateHeader", "Result"))

				+ SHeaderRow::Column(WidgetHittestGridInternal::Header_WidgetInfo)
				.DefaultLabel(LOCTEXT("SourceHeader", "Source"))
			)
		]

#if WITH_EDITOR
		+SVerticalBox::Slot()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("SettingsEditor.CheckoutWarningBorder"))
			.BorderBackgroundColor(FColor(166, 137, 0))
			.Visibility_Lambda([](){ return GSlateHitTestGridDebugging ? EVisibility::Visible : EVisibility::Collapsed; })
			[
				SNew(STextBlock)
				.Text(LOCTEXT("GSlateHitTestGridDebuggingEnabled", "The console variable GSlateHitTestGridDebugging is enabled. That will prevent the Widget Reflector to control how the debug information is displayed."))
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				.ShadowColorAndOpacity(FLinearColor::Black.CopyWithNewOpacity(0.3f))
				.ShadowOffset(FVector2D::UnitVector)
			]
		]
#endif
	];

	SetPause(false);
}


SWidgetHittestGrid::~SWidgetHittestGrid()
{
	SetPause(true);
}

void SWidgetHittestGrid::SetPause(bool bNewPaused)
{
	FHittestGrid::OnFindNextFocusableWidgetExecuted.RemoveAll(this);
	FSlateDebugging::PaintDebugElements.RemoveAll(this);
	if (!bNewPaused)
	{
		FSlateDebugging::PaintDebugElements.AddRaw(this, &SWidgetHittestGrid::HandleDrawDebuggerVisual);
		FHittestGrid::OnFindNextFocusableWidgetExecuted.AddRaw(this, &SWidgetHittestGrid::HandleFindNextFocusableWidget);
	}
}

void SWidgetHittestGrid::SaveSettings()
{
	GConfig->SetInt(TEXT("WidgetReflector.Hittestgrid"), TEXT("DisplayGridFlag"), static_cast<int32>(DisplayGridFlags), *GEditorPerProjectIni);
	GConfig->SetBool(TEXT("WidgetReflector.Hittestgrid"), TEXT("bVisualizeOnNavigation"), bVisualizeOnNavigation, *GEditorPerProjectIni);
	GConfig->SetBool(TEXT("WidgetReflector.Hittestgrid"), TEXT("bRejectWidgetReflectorEvent"), bRejectWidgetReflectorEvent, *GEditorPerProjectIni);
}

void SWidgetHittestGrid::LoadSettings()
{
	int32 DisplayGridFlagsAsInt = 0;
	if (GConfig->GetInt(TEXT("WidgetReflector.Hittestgrid"), TEXT("DisplayGridFlag"), DisplayGridFlagsAsInt, *GEditorPerProjectIni))
	{
		DisplayGridFlags = static_cast<FHittestGrid::EDisplayGridFlags>(DisplayGridFlagsAsInt);
	}
	GConfig->GetBool(TEXT("WidgetReflector.Hittestgrid"), TEXT("bVisualizeOnNavigation"), bVisualizeOnNavigation, *GEditorPerProjectIni);
	GConfig->GetBool(TEXT("WidgetReflector.Hittestgrid"), TEXT("bRejectWidgetReflectorEvent"), bRejectWidgetReflectorEvent, *GEditorPerProjectIni);
}

TSharedRef<SWidget> SWidgetHittestGrid::ConstructNavigationDetail() const
{
	return SNew(SGridPanel)
	.FillColumn(1, 0.5f)
	.FillColumn(3, 0.5f)

	+ SGridPanel::Slot(0, 0)
	[
		SNew(STextBlock)
		.Text(LOCTEXT("FromLabel", "From: "))
	]
	+ SGridPanel::Slot(1, 0)
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(SHyperlink)
			.Text(this, &SWidgetHittestGrid::GetNavigationFromText)
			.OnNavigate(this, &SWidgetHittestGrid::HandleNavigationNavigate, NavigationFromWidget)
		]
	]
	+ SGridPanel::Slot(0, 1)
	[
		SNew(STextBlock)
		.Text(LOCTEXT("ToLabel", "Result: "))
	]
	+ SGridPanel::Slot(1, 1)
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(SHyperlink)
			.Text(this, &SWidgetHittestGrid::GetNavigationResultText)
			.OnNavigate(this, &SWidgetHittestGrid::HandleNavigationNavigate, NavigationFromResult)
		]
	]
	+ SGridPanel::Slot(0, 2)
	[
		SNew(STextBlock)
		.Text(LOCTEXT("DirectionLabel", "Direction: "))
	]
	+ SGridPanel::Slot(1, 2)
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(this, &SWidgetHittestGrid::GetNavigationDirectionText)
		]
	]

	+ SGridPanel::Slot(2, 0)
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("UserIndexLabel", "User Index: "))
		]
	]
	+ SGridPanel::Slot(3, 0)
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(this, &SWidgetHittestGrid::GetNavigationUserIndexText)
		]
	]
	+ SGridPanel::Slot(2, 1)
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("RuleWidgetLabel", "Rule Widget: "))
		]
	]
	+ SGridPanel::Slot(3, 1)
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(SHyperlink)
			.Text(this, &SWidgetHittestGrid::GetNavigationRuleWidgetText)
			.OnNavigate(this, &SWidgetHittestGrid::HandleNavigationNavigate, NavigationRuleWidgetWidget)
		]
	]
	+ SGridPanel::Slot(2, 2)
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BoundaryRuleLabel", "Boundary Rule: "))
		]
	]
	+ SGridPanel::Slot(3, 2)
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(this, &SWidgetHittestGrid::GetNavigationBoundaryRuleText)
		]
	];
}

void SWidgetHittestGrid::HandleDrawDebuggerVisual(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, FSlateWindowElementList& InOutDrawElements, int32& InOutLayerId)
{
	if (GSlateHitTestGridDebugging)
	{
		return;
	}

	TSharedPtr<SWindow> Window = WindowToDisplay.Pin();
	bool bDisplay = bDisplayAllWindows || (Window && Window.Get() == InOutDrawElements.GetPaintWindow());
	if (bDisplay)
	{
		++InOutLayerId;
		InOutDrawElements.GetPaintWindow()->GetHittestGrid().DisplayGrid(InOutLayerId, InAllottedGeometry, InOutDrawElements, DisplayGridFlags);
	}
}

void SWidgetHittestGrid::HandleFindNextFocusableWidget(const FHittestGrid* HittestGrid, const FHittestGrid::FDebuggingFindNextFocusableWidgetArgs& Info)
{
	if (bRejectWidgetReflectorEvent)
	{
		if (TSharedPtr<const SWidget> PinnedReflectorWidget = ReflectorWidget.Pin())
		{
			TSharedPtr<const SWidget> WidgetToTest = Info.StartingWidget.Widget;
			while (WidgetToTest && !WidgetToTest->Advanced_IsWindow())
			{
				if (PinnedReflectorWidget == WidgetToTest)
				{
					return;
				}

				WidgetToTest = WidgetToTest->GetParentWidget();
			}
		}
	}

	if (bVisualizeOnNavigation && OnVisualizeWidget.IsBound())
	{
		TSharedRef<const SWidget> Widget = Info.Result ? Info.Result.ToSharedRef() : Info.StartingWidget.Widget;
		FWidgetPath WidgetPath;
		if (FSlateApplication::Get().FindPathToWidget(Widget, WidgetPath))
		{
			OnVisualizeWidget.Execute(WidgetPath);
		}
	}

	NavigationFrom = FWidgetReflectorNodeUtils::GetWidgetTypeAndShortName(Info.StartingWidget.Widget);
	NavigationFromWidget = Info.StartingWidget.Widget;
	NavigationResult = Info.Result ? FWidgetReflectorNodeUtils::GetWidgetTypeAndShortName(Info.Result) : FText::GetEmpty();
	NavigationFromResult = Info.Result;
	NavigationDirection = UEnum::GetDisplayValueAsText(Info.Direction);
	NavigationUserIndex = FText::AsNumber(Info.UserIndex);
	NavigationRuleWidget = FWidgetReflectorNodeUtils::GetWidgetTypeAndShortName(Info.RuleWidget.Widget);
	NavigationRuleWidgetWidget = Info.RuleWidget.Widget;
	NavigationBoundaryRule = UEnum::GetDisplayValueAsText(Info.NavigationReply.GetBoundaryRule());

	IntermediateResultNodesRoot.Reset();

	// Build all necessary widgets
	int32 Counter = 1;
	for (const FHittestGrid::FDebuggingFindNextFocusableWidgetArgs::FWidgetResult& Intermediate : Info.IntermediateResults)
	{
		TSharedRef<WidgetHittestGridInternal::FIntermediateResultNode> Node = MakeShared<WidgetHittestGridInternal::FIntermediateResultNode>();
		Node->Widget = Intermediate.Widget;
		Node->Index = Counter;
		Node->Result = Intermediate.Result;
		IntermediateResultNodesRoot.Add(Node);
		++Counter;
	}

	NavigationIntermediateResultTree->RebuildList();
}

namespace WidgetHittestGridText
{
	static const FText DisplayNone = LOCTEXT("None", "None");
	static const FText DisplayAllHitTest = LOCTEXT("AllHitTestGrid", "All Hit Test Grid");
}

FText SWidgetHittestGrid::GetDisplayButtonText() const
{
	if (bDisplayAllWindows)
	{
		return WidgetHittestGridText::DisplayAllHitTest;
	}
	if (TSharedPtr<SWindow> WindowToDisplayPinned = WindowToDisplay.Pin())
	{
		return WindowToDisplay.Pin()->GetTitle();
	}
	return WidgetHittestGridText::DisplayNone;
}

TSharedRef<SWidget> SWidgetHittestGrid::GetDisplayMenuContent()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	TWeakPtr<SWindow> EmptyWeakWindow;
	MenuBuilder.AddMenuEntry(
		WidgetHittestGridText::DisplayNone,
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SWidgetHittestGrid::HandleDisplayButtonClicked, EmptyWeakWindow),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SWidgetHittestGrid::HandleIsDisplayButtonChecked, EmptyWeakWindow)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		WidgetHittestGridText::DisplayAllHitTest,
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SWidgetHittestGrid::HandleDisplayAllClicked),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SWidgetHittestGrid::HandleIsDisplayAllChecked)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.BeginSection("Windows", LOCTEXT("Windows", "Windows"));
	{
		TArray<TSharedRef<SWindow>> VisibleWindows;
		FSlateApplication::Get().GetAllVisibleWindowsOrdered(VisibleWindows);
		for (TSharedRef<SWindow> Window : VisibleWindows)
		{
			TWeakPtr<SWindow> WeakWindow = Window;
			MenuBuilder.AddMenuEntry(
				Window->GetTitle(),
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SWidgetHittestGrid::HandleDisplayButtonClicked, WeakWindow),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SWidgetHittestGrid::HandleIsDisplayButtonChecked, WeakWindow)
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SWidgetHittestGrid::HandleDisplayButtonClicked(TWeakPtr<SWindow> Window)
{
	WindowToDisplay = Window;
	bDisplayAllWindows = false;
}

void SWidgetHittestGrid::HandleDisplayAllClicked()
{
	WindowToDisplay.Reset();
	bDisplayAllWindows = true;
}

bool SWidgetHittestGrid::HandleIsDisplayButtonChecked(TWeakPtr<SWindow> Window) const
{
	return bDisplayAllWindows == false && Window == WindowToDisplay;
}

TSharedRef<SWidget> SWidgetHittestGrid::GetFlagsMenuContent()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	auto AddEntry = [this, &MenuBuilder](const FText& Text, FHittestGrid::EDisplayGridFlags Flag)
	{
		MenuBuilder.AddMenuEntry(
			Text,
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SWidgetHittestGrid::HandleDisplayFlagsButtonClicked, Flag),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SWidgetHittestGrid::HandleIsDisplayFlagsButtonChecked, Flag)
			),
			NAME_None,
			EUserInterfaceActionType::Check);
	};

	AddEntry(LOCTEXT("HideDisabledWidget", "Hide Disabled Widgets"), FHittestGrid::EDisplayGridFlags::HideDisabledWidgets);
	AddEntry(LOCTEXT("HideUnsupportedKeyboardFocusWidget", "Hide Unsupported Keyboard Focus Widgets"), FHittestGrid::EDisplayGridFlags::HideUnsupportedKeyboardFocusWidgets);
	AddEntry(LOCTEXT("PaintwithFocusBrush", "Paint with Focus Brush"), FHittestGrid::EDisplayGridFlags::UseFocusBrush);

	return MenuBuilder.MakeWidget();
}

void SWidgetHittestGrid::HandleDisplayFlagsButtonClicked(FHittestGrid::EDisplayGridFlags Flag)
{
	DisplayGridFlags ^= Flag;

	SaveSettings();
}

bool SWidgetHittestGrid::HandleIsDisplayFlagsButtonChecked(FHittestGrid::EDisplayGridFlags Flag) const
{
	return EnumHasAnyFlags(DisplayGridFlags, Flag);
}

void SWidgetHittestGrid::HandleVisualizeOnNavigationChanged(ECheckBoxState InCheck)
{
	bool bNewValue = InCheck == ECheckBoxState::Checked;
	if (bVisualizeOnNavigation != bNewValue)
	{
		bVisualizeOnNavigation = bNewValue;
		SaveSettings();
	}
}

void SWidgetHittestGrid::HandleRejectWidgetReflectorChanged(ECheckBoxState InCheck)
{
	bool bNewValue = InCheck == ECheckBoxState::Checked;
	if (bRejectWidgetReflectorEvent != bNewValue)
	{
		bRejectWidgetReflectorEvent = bNewValue;
		SaveSettings();
	}
}

void SWidgetHittestGrid::HandleNavigationNavigate(TWeakPtr<const SWidget> WidgetToNavigate) const
{
	if (TSharedPtr<const SWidget> PinnedWidget = WidgetToNavigate.Pin())
	{
		OnWidgetSelected.ExecuteIfBound(PinnedWidget);
	}
}

TSharedRef<ITableRow> SWidgetHittestGrid::HandleWidgetHittestGridGenerateRow(TSharedRef<WidgetHittestGridInternal::FIntermediateResultNode> InReflectorNode, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(WidgetHittestGridInternal::SIntermediateResultNode, OwnerTable)
		.NodeToVisualize(InReflectorNode);
}

void SWidgetHittestGrid::HandleWidgetHittestGridSelectionChanged(TSharedPtr<WidgetHittestGridInternal::FIntermediateResultNode> Node, ESelectInfo::Type /*SelectInfo*/)
{
	if (Node)
	{
		if (TSharedPtr<const SWidget> PinnedWidget = Node->Widget.Pin())
		{
			OnWidgetSelected.ExecuteIfBound(PinnedWidget);
		}
	}
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_SLATE_DEBUGGING
