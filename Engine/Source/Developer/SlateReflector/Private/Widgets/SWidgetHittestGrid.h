// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Debugging/SlateDebugging.h"

#if WITH_SLATE_DEBUGGING

#include "Input/HittestGrid.h"
#include "Input/Reply.h"

#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"

class SWidget;
class SWindow;

enum class ECheckBoxState : uint8;
class ITableRow;
template<typename ItemType>
class SListView;
class STableViewBase;

namespace WidgetHittestGridInternal
{
	struct FIntermediateResultNode;
}
typedef SListView<TSharedRef<WidgetHittestGridInternal::FIntermediateResultNode>> SWidgetHittestGridTree;

/**
 * 
 */
class SWidgetHittestGrid : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnWidgetSelected, TSharedPtr<const SWidget>);
	DECLARE_DELEGATE_OneParam(FOnVisualizeWidget, const FWidgetPath&);
	SLATE_BEGIN_ARGS(SWidgetHittestGrid)
		{}
		SLATE_EVENT(FOnWidgetSelected, OnWidgetSelected)
		SLATE_EVENT(FOnVisualizeWidget, OnVisualizeWidget)
	SLATE_END_ARGS()

	virtual ~SWidgetHittestGrid();

	void Construct(const FArguments& InArgs, TSharedPtr<const SWidget> InReflectorWidget);

	/** Stop listening to new events */
	void SetPause(bool bNewPaused);

private:
	void LoadSettings();
	void SaveSettings();
	TSharedRef<SWidget> ConstructNavigationDetail() const;

	void HandleDrawDebuggerVisual(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, FSlateWindowElementList& InOutDrawElements, int32& InOutLayerId);
	void HandleFindNextFocusableWidget(const FHittestGrid* HittestGrid, const FHittestGrid::FDebuggingFindNextFocusableWidgetArgs& Info);

	FText GetDisplayButtonText() const;
	TSharedRef<SWidget> GetDisplayMenuContent();
	void HandleDisplayButtonClicked(TWeakPtr<SWindow> Window);
	void HandleDisplayAllClicked();
	bool HandleIsDisplayButtonChecked(TWeakPtr<SWindow> Window) const;
	bool HandleIsDisplayAllChecked() const { return bDisplayAllWindows; }

	TSharedRef<SWidget> GetFlagsMenuContent();
	void HandleDisplayFlagsButtonClicked(FHittestGrid::EDisplayGridFlags Flag);
	bool HandleIsDisplayFlagsButtonChecked(FHittestGrid::EDisplayGridFlags Flag) const;

	void HandleVisualizeOnNavigationChanged(ECheckBoxState InCheck);
	ECheckBoxState HandleGetVisualizeOnNavigationChecked() const { return bVisualizeOnNavigation ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; }

	void HandleRejectWidgetReflectorChanged(ECheckBoxState InCheck);
	ECheckBoxState HandleGetRejectWidgetReflectorChecked() const { return bRejectWidgetReflectorEvent ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; }

	FText GetNavigationFromText() const { return NavigationFrom; }
	FText GetNavigationResultText() const { return NavigationResult; }
	FText GetNavigationDirectionText() const { return NavigationDirection; }
	FText GetNavigationUserIndexText() const { return NavigationUserIndex; }
	FText GetNavigationRuleWidgetText() const { return NavigationRuleWidget; }
	FText GetNavigationBoundaryRuleText() const { return NavigationBoundaryRule; }
	void HandleNavigationNavigate(TWeakPtr<const SWidget> WidgetToNavigate) const;

	TSharedRef<ITableRow> HandleWidgetHittestGridGenerateRow(TSharedRef<WidgetHittestGridInternal::FIntermediateResultNode> InReflectorNode, const TSharedRef<STableViewBase>& OwnerTable);
	void HandleWidgetHittestGridSelectionChanged(TSharedPtr<WidgetHittestGridInternal::FIntermediateResultNode>, ESelectInfo::Type /*SelectInfo*/);

private:
	FOnWidgetSelected OnWidgetSelected;
	FOnVisualizeWidget OnVisualizeWidget;
	TWeakPtr<const SWidget> ReflectorWidget;

	FHittestGrid::EDisplayGridFlags DisplayGridFlags;
	TWeakPtr<SWindow> WindowToDisplay;
	bool bDisplayAllWindows;
	bool bVisualizeOnNavigation;
	bool bRejectWidgetReflectorEvent;

	TSharedPtr<SWidgetHittestGridTree> NavigationIntermediateResultTree;
	TArray<TSharedRef<WidgetHittestGridInternal::FIntermediateResultNode>> IntermediateResultNodesRoot;
	FText NavigationFrom;
	FText NavigationResult;
	FText NavigationDirection;
	FText NavigationUserIndex;
	FText NavigationRuleWidget;
	FText NavigationBoundaryRule;
	TWeakPtr<const SWidget> NavigationFromWidget;
	TWeakPtr<const SWidget> NavigationFromResult;
	TWeakPtr<const SWidget> NavigationRuleWidgetWidget;
};

#endif // WITH_SLATE_DEBUGGING