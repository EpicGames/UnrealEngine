// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SWidgetList.h"

#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformProcess.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "SPositiveActionButton.h"
#include "SSimpleButton.h"
#include "Styling/StyleColors.h"
#include "Types/ReflectionMetadata.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "WidgetList"

struct FWidgetListEntry
{
	FName Type;
	FName Location;
	FAssetData AssetData;
	uint64 Count = 0;
};

class SWidgetListRow : public SMultiColumnTableRow<TSharedPtr<FWidgetListEntry>>
{
public:
	SLATE_BEGIN_ARGS(SWidgetListRow) {}
		SLATE_ARGUMENT(FAccessSourceCode, OnAccessSource)
		SLATE_ARGUMENT(FAccessAsset, OnAccessAsset)
	SLATE_END_ARGS()

	static const FName NAME_Type;
	static const FName NAME_Location;
	static const FName NAME_Count;

	void Construct(const FArguments& InArgs, const TSharedPtr<FWidgetListEntry>& InEntry, const TSharedRef<STableViewBase>& OwnerTable)
	{
		Entry = InEntry;
		OnAccessSource = InArgs._OnAccessSource;
		OnAccessAsset = InArgs._OnAccessAsset;

		FSuperRowType::FArguments SuperArgs = FSuperRowType::FArguments();
		FSuperRowType::Construct(SuperArgs, OwnerTable);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == NAME_Type)
		{
			return SNew(STextBlock)
				.Text(FText::FromName(Entry->Type));
		}
		if (ColumnName == NAME_Location)
		{
			const FString ReadableLocation = FString::Printf(TEXT("%s(%d)"), *FPaths::GetCleanFilename(Entry->Location.GetPlainNameString()), Entry->Location.GetNumber());
			return SNew(SBox)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(FMargin(2.0f, 0.0f))
				[
					SNew(SHyperlink)
					.Text(FText::FromString(ReadableLocation))
					.OnNavigate(this, &SWidgetListRow::OnNavigateToSource)
				];
		}
		else if (ColumnName == NAME_Count)
		{
			return SNew(STextBlock)
				.Text(FText::AsNumber(Entry->Count));
		}

		return SNullWidget::NullWidget;
	}

private:
	void OnNavigateToSource()
	{
		if (OnAccessAsset.IsBound() && Entry->AssetData.IsValid())
		{
			Entry->AssetData.GetPackage();
			OnAccessAsset.Execute(Entry->AssetData.GetAsset());
		}

		if (OnAccessAsset.IsBound())
		{
			OnAccessSource.Execute(Entry->Location.GetPlainNameString(), Entry->Location.GetNumber(), 0);
		}
	}

private:
	TSharedPtr<FWidgetListEntry> Entry;
	FAccessSourceCode OnAccessSource;
	FAccessAsset OnAccessAsset;
};

const FName SWidgetListRow::NAME_Type = "Type";
const FName SWidgetListRow::NAME_Location = "Location";
const FName SWidgetListRow::NAME_Count = "Count";

void SWidgetList::Construct(const FArguments& Args)
{
	OnAccessAsset = Args._OnAccessAsset;
	OnAccessSource = Args._OnAccessSource;

#if UE_WITH_SLATE_DEBUG_WIDGETLIST
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(4, 6)
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FMargin(0, 0, 10, 0))
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(STextBlock)
					.Text_Lambda([this]() { return FText::Format(LOCTEXT("CurrentFormat", "Current: {0}"), CurrentCount); })
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(STextBlock)
					.Text_Lambda([this]() { return FText::Format(LOCTEXT("PreviousFormat", "Previous: {0}"), PreviousCount); })
				]
				+ SUniformGridPanel::Slot(2, 0)
				[
					SNew(STextBlock)
					.Text_Lambda([this]() 
					{
						FNumberFormattingOptions Options = FNumberFormattingOptions().SetAlwaysSign(true);
						return FText::Format(LOCTEXT("CurrentFormat", "Current: {0}"), FText::AsNumber(CurrentCount - PreviousCount, &Options));
					})
					.ColorAndOpacity_Lambda([this]()
					{
						const int32 Diff = CurrentCount - PreviousCount;
						return Diff > 0 ? FStyleColors::AccentGreen :
							(Diff < 0 ? FStyleColors::AccentRed :
							FStyleColors::Foreground); 
					})
				]
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(0,0,4,0)
			.AutoWidth()
			[
				SNew(SSimpleButton)
				.Icon(FAppStyle::Get().GetBrush("Icons.Save"))
				.Text(LOCTEXT("WidgetListExport", "Export to CSV..."))
				.OnClicked(this, &SWidgetList::ExportToCSV)
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SPositiveActionButton)
				.Icon(FAppStyle::Get().GetBrush("Icons.Refresh"))
				.Text(LOCTEXT("WidgetListRefresh", "Refresh"))
				.OnClicked_Lambda([this]() { Refresh(); return FReply::Handled(); })
			]
		]
		+ SVerticalBox::Slot()
		[
			SAssignNew(ListView, SListView<TSharedPtr<FWidgetListEntry>>)
			.ListItemsSource(&Entries)
			.OnGenerateRow(this, &SWidgetList::OnGenerateRow)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+ SHeaderRow::Column(SWidgetListRow::NAME_Type)
				.FillWidth(0.3)
				.DefaultLabel(LOCTEXT("WidgetListWidget", "Type"))
				.OnSort(this, &SWidgetList::OnSort)

				+ SHeaderRow::Column(SWidgetListRow::NAME_Location)
				.FillWidth(0.5)
				.DefaultLabel(LOCTEXT("WidgetListLocation", "Location"))
				.OnSort(this, &SWidgetList::OnSort)

				+ SHeaderRow::Column(SWidgetListRow::NAME_Count)
				.FillWidth(0.2)
				.DefaultLabel(LOCTEXT("WidgetListCount", "Count"))
				.OnSort(this, &SWidgetList::OnSort)
			)
		]
	];
#else

	ChildSlot
	[
		SNew(SBox)
		.Padding(10)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("WidgetListNotDefined", "Widget list requires UE_WITH_SLATE_DEBUG_WIDGETLIST to be defined."))
			.ColorAndOpacity(FStyleColors::Error)
		]
	];

#endif // UE_WITH_SLATE_DEBUG_WIDGETLIST
}

void SWidgetList::Refresh()
{
#if WITH_SLATE_DEBUGGING
	TArray<const SWidget*> Widgets = FSlateDebugging::GetAllWidgets();

	PreviousCount = CurrentCount;
	CurrentCount = Widgets.Num();

	using FWidgetKey = TPair<FName, FName>;
	TMap<FWidgetKey, TSharedPtr<FWidgetListEntry>> WidgetEntryMap;

	for (const SWidget* Widget : Widgets)
	{
		FWidgetKey Key { Widget->GetType(), Widget->GetCreatedInLocation() };
		TSharedPtr<FWidgetListEntry>* Entry = WidgetEntryMap.Find(Key);
		if (Entry == nullptr)
		{
			Entry = &WidgetEntryMap.Add(Key, MakeShared<FWidgetListEntry>());
			(*Entry)->Type = Widget->GetType();
			(*Entry)->Location = Widget->GetCreatedInLocation();
			(*Entry)->Count = 0;

			TSharedPtr<FReflectionMetaData> WidgetMeta = Widget->GetMetaData<FReflectionMetaData>();
			if (WidgetMeta.IsValid() && WidgetMeta->Asset.Get() != nullptr)
			{
				(*Entry)->AssetData = FAssetData(WidgetMeta->Asset.Get());
			}
		}

		(*Entry)->Count++;
	}

	Entries.Reset();

	for (const TPair<FWidgetKey, TSharedPtr<FWidgetListEntry>>& EntryPair : WidgetEntryMap)
	{
		Entries.Add(EntryPair.Value);
	}

	Sort();
#endif //WITH_SLATE_DEBUGGING
}

FReply SWidgetList::ExportToCSV() const
{
#if WITH_SLATE_DEBUGGING
	FString OutputFile;
	const TCHAR* DefaultFile = TEXT("WidgetList.csv");

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		const void* ParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

		TArray<FString> SaveFilenames;

		bool bSave = DesktopPlatform->SaveFileDialog(
			ParentWindowHandle,
			LOCTEXT("ExportToCSVTitle", "Export...").ToString(),
			FPaths::ProjectSavedDir(),
			DefaultFile,
			TEXT("CSV|*.csv"),
			EFileDialogFlags::None,
			SaveFilenames
		);

		if (!bSave || SaveFilenames.Num() == 0)
		{
			return FReply::Handled();
		}

		OutputFile = SaveFilenames[0];
	}
	else
	{
		OutputFile = FPaths::Combine(FPaths::ProjectSavedDir(), TEXT("WidgetList.csv"));
	}

	FSlateDebugging::ExportWidgetList(OutputFile);

	FPlatformProcess::ExploreFolder(*OutputFile);
#endif //WITH_SLATE_DEBUGGING

	return FReply::Handled();
}

void SWidgetList::Sort()
{
	if (SortColumn.IsNone())
	{
		SortColumn = SWidgetListRow::NAME_Count;
	}

	if (SortColumn.IsEqual(SWidgetListRow::NAME_Count))
	{
		Entries.Sort([this](const TSharedPtr<FWidgetListEntry>& A, const TSharedPtr<FWidgetListEntry>& B)
			{
				return bSortAscending ?
					A->Count < B->Count :
					A->Count > B->Count;
			});
	}
	else if (SortColumn.IsEqual(SWidgetListRow::NAME_Type))
	{
		Entries.Sort([this](const TSharedPtr<FWidgetListEntry>& A, const TSharedPtr<FWidgetListEntry>& B)
			{
				return bSortAscending ?
					A->Type.LexicalLess(B->Type) :
					B->Type.LexicalLess(A->Type);
			});
	}
	else if (SortColumn.IsEqual(SWidgetListRow::NAME_Location))
	{
		Entries.Sort([this](const TSharedPtr<FWidgetListEntry>& A, const TSharedPtr<FWidgetListEntry>& B)
			{
				return bSortAscending ?
					A->Location.LexicalLess(B->Location) :
					B->Location.LexicalLess(A->Location);
			});
	}

	if (ListView.IsValid())
	{
		ListView->RequestListRefresh();
	}
}

void SWidgetList::OnSort(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode)
{
	if (SortPriority == EColumnSortPriority::Primary &&
		NewSortMode != EColumnSortMode::None)
	{
		bSortAscending = NewSortMode == EColumnSortMode::Ascending;
		SortColumn = ColumnId;

		Sort();
	}
}

TSharedRef<ITableRow> SWidgetList::OnGenerateRow(TSharedPtr<FWidgetListEntry> Entry, const TSharedRef<STableViewBase>& Table) const
{
	return SNew(SWidgetListRow, Entry, Table)
		.OnAccessSource(OnAccessSource)
		.OnAccessAsset(OnAccessAsset);
}

#undef LOCTEXT_NAMESPACE
