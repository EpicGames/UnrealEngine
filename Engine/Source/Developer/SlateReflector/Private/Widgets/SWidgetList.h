// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Debugging/SlateDebugging.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

struct FWidgetListEntry;

class SWidgetList : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SWidgetList) {}
		SLATE_EVENT(FAccessAsset, OnAccessAsset)
		SLATE_EVENT(FAccessSourceCode, OnAccessSource)
	SLATE_END_ARGS()

public:

	void Construct(const FArguments& Args);

private:
	void Refresh();
	void Sort();
	void OnSort(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode);
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FWidgetListEntry> Entry, const TSharedRef<STableViewBase>& Table) const;
	FReply ExportToCSV() const;

private:
	TSharedPtr<SListView<TSharedPtr<FWidgetListEntry>>> ListView;
	TArray<TSharedPtr<FWidgetListEntry>> Entries;
	FAccessAsset OnAccessAsset;
	FAccessSourceCode OnAccessSource;
	int32 CurrentCount = 0;
	int32 PreviousCount = 0;
	FName SortColumn;
	bool bSortAscending = false;
};
