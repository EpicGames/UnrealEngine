// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SWidgetReflector.h"

#include "ISlateReflectorModule.h"
#include "SlateReflectorModule.h"
#include "Rendering/DrawElements.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "InputEventVisualizer.h"
#include "Styling/WidgetReflectorStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SSlateOptions.h"
#include "Widgets/SWidgetReflectorTreeWidgetItem.h"
#include "Widgets/SWidgetReflectorToolTipWidget.h"
#include "Widgets/SWidgetEventLog.h"
#include "Widgets/SWidgetHittestGrid.h"
#include "Widgets/SWidgetList.h"
#include "Widgets/SInvalidationPanel.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SVisualElementsTab.h"
#include "Widgets/SWidgetSearchPanel.h"
#include "Widgets/SWidgetSnapshotVisualizer.h"
#include "WidgetSnapshotService.h"
#include "Types/ReflectionMetadata.h"
#include "Debugging/SlateDebugging.h"
#include "VisualTreeCapture.h"
#include "Models/WidgetReflectorNode.h"
#include "Fonts/FontCache.h"
#include "Fonts/FontTypes.h"
#include "IImageWrapperModule.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Textures/SlateShaderResource.h"

#if SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM
#include "DesktopPlatformModule.h"
#endif // SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM

#if SLATE_REFLECTOR_HAS_SESSION_SERVICES
#include "ISessionManager.h"
#include "ISessionServicesModule.h"
#endif // SLATE_REFLECTOR_HAS_SESSION_SERVICES

#if WITH_EDITOR
#include "Editor.h"
#include "Framework/Docking/LayoutService.h"
#include "PropertyEditorModule.h"
#endif

#define LOCTEXT_NAMESPACE "SWidgetReflector"

/**
 * Widget reflector user widget.
 */

namespace WidgetReflectorCVars
{
	static bool bEnableFocusOnPick = true;
	static FAutoConsoleVariableRef EnableFocusOnPick(
		TEXT("Slate.EnableFocusOnPick"),
		bEnableFocusOnPick,
		TEXT("If true, Widget Reflector window will be automatically focused when a pick is made.")
	);
}

/* Local helpers
 *****************************************************************************/
 
namespace WidgetReflectorImpl
{

/** Command to take a snapshot. */
void TakeSnapshotCommand(const TArray<FString>&);

void DumpFontAtlasesCommand(const TArray<FString>&);

FAutoConsoleCommand CCommandWidgetReflectorTakeSnapshot(
	TEXT("WidgetReflector.TakeSnapshot")
	, TEXT("Take a snapshot and save the result on the local drive. ie. WidgetReflector.TakeSnapshot [Delay=1.0] [Navigation=false]")
	, FConsoleCommandWithArgsDelegate::CreateStatic(&TakeSnapshotCommand));

FAutoConsoleCommand CCommandDumpFontAtlases(
	TEXT("WidgetReflector.DumpFontAtlases")
	, TEXT("Save all of the font atlases currently in the font cache on the local drive. A timestamp will be automatically appended to the directory where the textures will be saved. The default location is YourProject/Saved/WidgetReflector/FontAtlases_Timestamp ie. WidgetReflector.DumpFontAtlases[SaveDirectory=D:\\My\\Path] [SaveFilePrefix=ExamplePrefix]")
	, FConsoleCommandWithArgsDelegate::CreateStatic(&DumpFontAtlasesCommand));

/** Information about a potential widget snapshot target */
struct FWidgetSnapshotTarget
{
	/** Display name of the target (used in the UI) */
	FText DisplayName;

	/** Instance ID of the target */
	FGuid InstanceId;
};

/** Different UI modes the widget reflector can be in */
enum class EWidgetReflectorUIMode : uint8
{
	Live,
	Snapshot,
};

namespace WidgetReflectorTabID
{
	static const FName WidgetHierarchy = "WidgetReflector.WidgetHierarchyTab";
	static const FName SnapshotWidgetPicker = "WidgetReflector.SnapshotWidgetPickerTab";
	static const FName WidgetDetails = "WidgetReflector.WidgetDetailsTab";
	static const FName SlateOptions = "WidgetReflector.SlateOptionsTab";
	static const FName WidgetEvents = "WidgetReflector.WidgetEventsTab";
	static const FName HittestGrid = "WidgetReflector.HittestGridTab";
	static const FName WidgetList = "WidgetReflector.WidgetList";
	static const FName VisualElements = "WidgetReflector.VisualElements";
	static const FName WidgetSearch = "WidgetReflector.WidgetSearch";
}

namespace WidgetReflectorText
{
	static const FText HitTestPicking = LOCTEXT("PickHitTestable", "Pick Hit-Testable Widgets");
	static const FText VisualPicking = LOCTEXT("PickVisual", "Pick Painted Widgets");
	static const FText Focus = LOCTEXT("ShowFocus", "Show Focus");
	static const FText Focusing = LOCTEXT("ShowingFocus", "Showing Focus (Esc to Stop)");
	static const FText Picking = LOCTEXT("PickingWidget", "Picking (Esc to Stop)");
}

namespace WidgetReflectorIcon
{
	static const FName FocusPicking = "Icon.FocusPicking";
	static const FName HitTestPicking = "Icon.HitTestPicking";
	static const FName VisualPicking = "Icon.VisualPicking";
	static const FName Ellipsis = "Icon.Ellipsis";
	static const FName Filter = "Icon.Filter";
	static const FName LoadSnapshot = "Icon.LoadSnapshot";
	static const FName TakeSnapshot = "Icon.TakeSnapshot";
}

namespace VisualCaptureEnabled
{
	static const FName WidgetPickingMode = "WidgetPickingMode";
}

enum class EWidgetPickingMode : uint8
{
	None = 0,
	Focus,
	HitTesting,
	Drawable
};

EWidgetPickingMode ConvertToWidgetPickingMode(int32 Number)
{
	if (Number < 0 || Number > static_cast<int32>(EWidgetPickingMode::Drawable))
	{
		return EWidgetPickingMode::None;
	}
	return static_cast<EWidgetPickingMode>(Number);
}

/**
 * Widget reflector implementation
 */
class SWidgetReflector : public ::SWidgetReflector
{
	// The reflector uses a tree that observes FWidgetReflectorNodeBase objects.
	typedef STreeView<TSharedRef<FWidgetReflectorNodeBase>> SReflectorTree;

	//~ Begin ::SWidgetReflector implementation
public:
	virtual ~SWidgetReflector();

	virtual void UpdateColumnWidth(TSharedRef<FWidgetReflectorNodeBase> InItem, FName InColumn) override;

private:
	virtual void Construct( const FArguments& InArgs) override;
	//~ End ::SWidgetReflector implementation


	void HandlePullDownAtlasesMenu(FMenuBuilder& MenuBuilder);
	void HandlePullDownWindowMenu(FMenuBuilder& MenuBuilder);

	TSharedRef<SDockTab> SpawnSlateOptionWidgetTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnWidgetHierarchyTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnSnapshotWidgetPicker(const FSpawnTabArgs& Args);

#if WITH_EDITOR
	TSharedRef<SDockTab> SpawnWidgetDetails(const FSpawnTabArgs& Args);
#endif

#if WITH_SLATE_DEBUGGING
	TSharedRef<SDockTab> SpawnWidgetEvents(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnWidgetHittestGrid(const FSpawnTabArgs& Args);
#endif

	TSharedRef<SDockTab> SpawnWidgetList(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnVisualElementsTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnWidgetSearchTab(const FSpawnTabArgs& Args);

	float GetColumnWidth(FName InColumn, float InDefault) const;

public:

	void HandleTabManagerPersistLayout(const TSharedRef<FTabManager::FLayout>& LayoutToSave);
	void OnTabSpawned(const FName& TabIdentifier, const TSharedRef<SDockTab>& SpawnedTab);
	void CloseTab(const FName& TabIdentifier);

	void SaveSettings();
	void LoadSettings();

	void SetUIMode(const EWidgetReflectorUIMode InNewMode);

	//~ Begin SCompoundWidget overrides
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	//~ End SCompoundWidget overrides

	//~ Begin IWidgetReflector interface
	virtual bool IsInPickingMode() const override
	{
		return PickingMode != EWidgetPickingMode::None;
	}

	virtual bool IsShowingFocus() const override
	{
		return PickingMode == EWidgetPickingMode::Focus;
	}

	virtual bool IsVisualizingLayoutUnderCursor() const override
	{
		return PickingMode == EWidgetPickingMode::HitTesting || PickingMode == EWidgetPickingMode::Drawable;
	}

	virtual void OnWidgetPicked() override
	{
		SetPickingMode(EWidgetPickingMode::None);
	}

	virtual bool ReflectorNeedsToDrawIn( TSharedRef<SWindow> ThisWindow ) const override;

	virtual void SetSourceAccessDelegate( FAccessSourceCode InDelegate ) override
	{
		SourceAccessDelegate = InDelegate;
	}

	bool HandleSourceAccessDelegate(const FString& FileName, int32 LineNumber, int32 ColumnNumber) const
	{
		if (SourceAccessDelegate.IsBound())
		{
			return SourceAccessDelegate.Execute(FileName, LineNumber, ColumnNumber);
		}
		return false;
	}

	virtual void SetAssetAccessDelegate(FAccessAsset InDelegate) override
	{
		AssetAccessDelegate = InDelegate;
	}

	bool HandleAssetAccessDelegate(UObject* Asset) const
	{
		if (AssetAccessDelegate.IsBound())
		{
			return AssetAccessDelegate.Execute(Asset);
		}
		return false;
	}
	
	virtual void SetDebugObjectAccessDelegate(FAccessDebugObject InDelegate)override
	{
		DebugObjectAccessDelegate = InDelegate;
	}

	virtual void SetWidgetsToVisualize( const FWidgetPath& InWidgetsToVisualize ) override;
	virtual int32 Visualize( const FWidgetPath& InWidgetsToVisualize, FSlateWindowElementList& OutDrawElements, int32 LayerId ) override;
	//~ End IWidgetReflector interface

	/**
	 * Generates a tool tip for the given reflector tree node.
	 *
	 * @param InReflectorNode The node to generate the tool tip for.
	 * @return The tool tip widget.
	 */
	TSharedPtr<IToolTip> GenerateToolTipForReflectorNode( TWeakPtr<FWidgetReflectorNodeBase> InReflectorNode ) const;

	/**
	 * Mark the provided reflector nodes such that they stand out in the tree and are visible.
	 *
	 * @param WidgetPathToObserve The nodes to mark.
	 */
	void VisualizeAsTree( const TArray< TSharedRef<FWidgetReflectorNodeBase> >& WidgetPathToVisualize );

	/**
	 * Draw the widget path to the picked widget as the widgets' outlines.
	 *
	 * @param InWidgetsToVisualize A widget path whose widgets' outlines to draw.
	 * @param OutDrawElements A list of draw elements; we will add the output outlines into it.
	 * @param LayerId The maximum layer achieved in OutDrawElements so far.
	 * @return The maximum layer ID we achieved while painting.
	 */
	int32 VisualizePickAsRectangles( const FWidgetPath& InWidgetsToVisualize, FSlateWindowElementList& OutDrawElements, int32 LayerId );

	/**
	 * Draw an outline for the specified nodes.
	 *
	 * @param InNodesToDraw A widget path whose widgets' outlines to draw.
	 * @param WindowGeometry The geometry of the window in which to draw.
	 * @param OutDrawElements A list of draw elements; we will add the output outlines into it.
	 * @param LayerId the maximum layer achieved in OutDrawElements so far.
	 * @return The maximum layer ID we achieved while painting.
	 */
	int32 VisualizeSelectedNodesAsRectangles( const TArray<TSharedRef<FWidgetReflectorNodeBase>>& InNodesToDraw, const TSharedRef<SWindow>& VisualizeInWindow, FSlateWindowElementList& OutDrawElements, int32 LayerId );

	/** Draw the actual highlight */
	void DrawWidgetVisualization(const FPaintGeometry& WidgetGeometry, FLinearColor Color, FSlateWindowElementList& OutDrawElements, int32& LayerId);

	/** Clear previous selection and set the selection to the live widget. */
	void SelectLiveWidget( TSharedPtr<const SWidget> InWidget );

	/** Clear previous selection and select the given node in the reflector tree by expanding its path and scrolling it into view. */
	void SelectReflectorNode(TSharedRef<FWidgetReflectorNodeBase> InNode);

	/** Get all selected live widgets. */
	TArray<TSharedPtr<const SWidget>> GetSelectedLiveWidgets() const;

	/** Returns true when Drawable picking mode is active. */
	bool IsDrawablePickingActive() const { return PickingMode == EWidgetPickingMode::Drawable; }

	/** Set the given nodes as the root of the tree. */
	void SetNodesAsReflectorTreeRoot(TArray<TSharedRef<FWidgetReflectorNodeBase>> RootNodes);

	/** Filter the selected nodes before setting them as root. */
	TArray<TSharedRef<FWidgetReflectorNodeBase>> FilterSelectedToSetAsReflectorTreeRoot(TArray<TSharedRef<FWidgetReflectorNodeBase>> RootNodes);

	/** Is there any selected node in the reflector tree. */
	bool DoesReflectorTreeHasSelectedItem() const { return SelectedNodes.Num() > 0; }

	/** Apply the requested filter to the reflected tree root. */
	void UpdateFilteredTreeRoot();

	void HandleDisplayTextureAtlases();
	void HandleDisplayFontAtlases();

	//~ Handle for the picking button
	ECheckBoxState HandleGetPickingButtonChecked() const;
	void HandlePickingModeStateChanged();
	FSlateIcon HandleGetPickingModeImage() const;
	FText HandleGetPickingModeText() const;
	TSharedRef<SWidget> HandlePickingModeContextMenu();	
	void HandlePickButtonClicked(EWidgetPickingMode InPickingMode);

	void SetPickingMode(EWidgetPickingMode InMode)
	{
		if (PickingMode != InMode)
		{
			// If leaving Drawable mode, remove its entry from the enablers.
			if (PickingMode == EWidgetPickingMode::Drawable)
			{
				DisableVisualCapture(VisualCaptureEnabled::WidgetPickingMode);
			}

			// Enable the picking mode.
			PickingMode = InMode;

			// If we're entering Drawable mode, add its entry to the enablers.
			if (PickingMode == EWidgetPickingMode::Drawable)
			{
				EnableVisualCapture(VisualCaptureEnabled::WidgetPickingMode);
			}
			else if (PickingMode != EWidgetPickingMode::None && !VisualCapture.IsEnabled())
			{
				VisualCapture.Reset();
			}
		}
	}

	/** Enable the VisualCapture. */
	void EnableVisualCapture(FName Enabler);

	/** Disable the VisualCapture if there are other enablers. */
	void DisableVisualCapture(FName Enabler);

	/** Callback to see whether the "Snapshot Target" combo should be enabled */
	bool IsSnapshotTargetComboEnabled() const;

	/** Callback to see whether the "Take Snapshot" button should be enabled */
	bool IsTakeSnapshotButtonEnabled() const;

	/** Callback for clicking the "Take Snapshot" button. */
	void HandleTakeSnapshotButtonClicked();

	/** Build option menu for snaphot. */
	TSharedRef<SWidget> HandleSnapshotOptionsTreeContextMenu();

	/** Takes a snapshot of the current state of the snapshot target. */
	void TakeSnapshot();

	/** Used as a callback for the "snapshot pending" notification item buttons, called when we should give up on a snapshot request */
	void OnCancelPendingRemoteSnapshot();

	/** Callback for when a remote widget snapshot is available */
	void HandleRemoteSnapshotReceived(const TArray<uint8>& InSnapshotData);

#if SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM
	/** Callback for clicking the "Load Snapshot" button. */
	void HandleLoadSnapshotButtonClicked();
#endif // SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM

	/** Called to update the list of available snapshot targets */
	void UpdateAvailableSnapshotTargets();

	/** Called to update the currently selected snapshot target (after the list has been refreshed) */
	void UpdateSelectedSnapshotTarget();

	/** Called when the list of available snapshot targets changes */
	void OnAvailableSnapshotTargetsChanged();

	/** Called when a node is set as root to create the breadcrum trail */
	void CreateCrumbTrailForNode(TSharedRef<FWidgetReflectorNodeBase> InReflectorNode);

	/** Get the display name of the currently selected snapshot target */
	FText GetSelectedSnapshotTargetDisplayName() const;

	/** Generate a row widget for the available targets combo box */
	TSharedRef<SWidget> HandleGenerateAvailableSnapshotComboItemWidget(TSharedPtr<FWidgetSnapshotTarget> InItem) const;
	
	/** Update the selected target when the combo box selection is changed */
	void HandleAvailableSnapshotComboSelectionChanged(TSharedPtr<FWidgetSnapshotTarget> InItem, ESelectInfo::Type InSeletionInfo);

	/** Callback for generating a row in the reflector tree view. */
	TSharedRef<ITableRow> HandleReflectorTreeGenerateRow( TSharedRef<FWidgetReflectorNodeBase> InReflectorNode, const TSharedRef<STableViewBase>& OwnerTable );

	/** Callback for getting the child items of the given reflector tree node. */
	void HandleReflectorTreeGetChildren( TSharedRef<FWidgetReflectorNodeBase> InReflectorNode, TArray<TSharedRef<FWidgetReflectorNodeBase>>& OutChildren );

	/** Callback for when the selection in the reflector tree has changed. */
	void HandleReflectorTreeSelectionChanged( TSharedPtr<FWidgetReflectorNodeBase>, ESelectInfo::Type /*SelectInfo*/ );

	/** Callback for when the context menu in the reflector tree is requested. */
	TSharedRef<SWidget> HandleReflectorTreeContextMenu();
	TSharedPtr<SWidget> HandleReflectorTreeContextMenuPtr();

	/** Callback for when an item in the reflector tree is clicked on. */
	void HandleReflectorTreeOnMouseClick(TSharedRef<FWidgetReflectorNodeBase> InReflectorNode);

	/** Callback for when a crumb is clicked on. */
	void HandleBreadcrumbOnClick(const TSharedRef<FWidgetReflectorNodeBase>& InReflectorNode);

	/** Callback for when the menu of a crumb delimiter is requested. */
	TSharedRef< SWidget > HandleBreadcrumbDelimiterMenu(const TSharedRef<FWidgetReflectorNodeBase>& InReflectorNode);

	/** Callback for when the reflector tree header list changed. */
	void HandleReflectorTreeHiddenColumnsListChanged();

	/** Reset the filtered tree root. */
	void HandleResetFilteredTreeRoot();

	/** Show the start of the UMG tree. */
	void HandleStartTreeWithUMG();

	/** Should we display the breadcrumb trail. */
	EVisibility HandleIsBreadcrumbVisible() const;

	/** Should we show only the UMG tree. */
	bool HandleIsStartTreeWithUMGEnabled() const { return bFilterReflectorTreeRootWithUMG; }

	bool HandleHasPendingActions() const { return !bIsPendingDelayedSnapshot; }

	/** Determine the text of Take Snapshot button */
	FText HandleGetTakeSnapshotText() const { return bIsPendingDelayedSnapshot ? LOCTEXT("CancelSnapshotButtonText", "Cancel Snapshot") : LOCTEXT("TakeSnapshotButtonText", "Take Snapshot"); }

#if WITH_EDITOR
	void HandleEndPIE(const bool bIsSimulating);
	void HandleObjectTransacted(UObject* InObject, const FTransactionObjectEvent& InTransactionEvent);
#endif //WITH_EDITOR

private:
	TSharedPtr<FTabManager> TabManager;
	TMap<FName, TWeakPtr<SDockTab>> SpawnedTabs;

	TSharedPtr<SReflectorTree> ReflectorTree;
	TArray<FString> HiddenReflectorTreeColumns;

	TSharedPtr<SBreadcrumbTrail<TSharedRef<FWidgetReflectorNodeBase>> > BreadCrumb;
	/** Node that are currently selected */
	TArray<TSharedRef<FWidgetReflectorNodeBase>> SelectedNodes;
	/** The original path of the widget picked. It may include node that are now hidden by the filter */
	TArray<TSharedRef<FWidgetReflectorNodeBase>> PickedWidgetPath;
	/** Root of the tree before filtering */
	TArray<TSharedRef<FWidgetReflectorNodeBase>> ReflectorTreeRoot;
	/** Root of the tree after filtering */
	TArray<TSharedRef<FWidgetReflectorNodeBase>> FilteredTreeRoot;

	/** When working with a snapshotted tree, this will contain the snapshot hierarchy and screenshot info */
	FWidgetSnapshotData SnapshotData;
	TSharedPtr<SWidgetSnapshotVisualizer> WidgetSnapshotVisualizer;

	/** List of available snapshot targets, as well as the one we currently have selected */
	TSharedPtr<SComboBox<TSharedPtr<FWidgetSnapshotTarget>>> AvailableSnapshotTargetsComboBox;
	TArray<TSharedPtr<FWidgetSnapshotTarget>> AvailableSnapshotTargets;
	FGuid SelectedSnapshotTargetInstanceId;
	TSharedPtr<FWidgetSnapshotService> WidgetSnapshotService;
	TWeakPtr<SNotificationItem> WidgetSnapshotNotificationPtr;
	FGuid RemoteSnapshotRequestId;

	FAccessSourceCode SourceAccessDelegate;
	FAccessAsset AssetAccessDelegate;
	FAccessDebugObject DebugObjectAccessDelegate;

	EWidgetReflectorUIMode CurrentUIMode;
	EWidgetPickingMode PickingMode;
	EWidgetPickingMode LastPickingMode;
	bool bFilterReflectorTreeRootWithUMG;

#if WITH_EDITOR
	TSharedPtr<IDetailsView> PropertyViewPtr;
#endif
#if WITH_SLATE_DEBUGGING
	TWeakPtr<SWidgetHittestGrid> WidgetHittestGrid;
#endif
	TSharedPtr<SVisualElementsTab> VisualElementsTabWidget;
	TWeakPtr<UE::SlateReflector::SWidgetSearchPanel> SearchPanel;

	FVisualTreeCapture VisualCapture;
	/** Controls whether VisualCapture is active. Each entry is a named reason for keeping it enabled. */
	TArray<FName> VisualCaptureEnablers;

private:
	float SnapshotDelay;
	bool bIsPendingDelayedSnapshot;
	bool bRequestNavigationSimulation;
#if WITH_EDITOR
	bool bTransactedPIEObject;
#endif
	double TimeOfScheduledSnapshot;

	TMap<FName, float> ColumnWidths;
};

void SWidgetReflector::Construct( const FArguments& InArgs )
{
	// If saved, LoadSettings will override these variables.
	LastPickingMode = EWidgetPickingMode::HitTesting;
	HiddenReflectorTreeColumns.Add(SReflectorTreeWidgetItem::NAME_Enabled.ToString());
	HiddenReflectorTreeColumns.Add(SReflectorTreeWidgetItem::NAME_Volatile.ToString());
	HiddenReflectorTreeColumns.Add(SReflectorTreeWidgetItem::NAME_HasActiveTimer.ToString());
	HiddenReflectorTreeColumns.Add(SReflectorTreeWidgetItem::NAME_ActualSize.ToString());
	HiddenReflectorTreeColumns.Add(SReflectorTreeWidgetItem::NAME_LayerId.ToString());

	LoadSettings();

	CurrentUIMode = EWidgetReflectorUIMode::Live;
	PickingMode = EWidgetPickingMode::None;
	bFilterReflectorTreeRootWithUMG = false;

	SnapshotDelay = 0.0f;
	bIsPendingDelayedSnapshot = false;
	bRequestNavigationSimulation = false;
#if WITH_EDITOR
	bTransactedPIEObject = false;
#endif 
	TimeOfScheduledSnapshot = -1.0;

	WidgetSnapshotService = InArgs._WidgetSnapshotService;

#if SLATE_REFLECTOR_HAS_SESSION_SERVICES
	{
		TSharedPtr<ISessionManager> SessionManager = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices").GetSessionManager();
		if (SessionManager.IsValid())
		{
			SessionManager->OnSessionsUpdated().AddSP(this, &SWidgetReflector::OnAvailableSnapshotTargetsChanged);
		}
	}
#endif // SLATE_REFLECTOR_HAS_SESSION_SERVICES
	SelectedSnapshotTargetInstanceId = FApp::GetInstanceId();
	UpdateAvailableSnapshotTargets();

	const FName TabLayoutName = "WidgetReflector_Layout_NoStats_v2";

	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout(TabLayoutName)
	->AddArea
	(
		FTabManager::NewPrimaryArea()
		->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewStack()
			->SetHideTabWell(true)
			->AddTab(WidgetReflectorTabID::SlateOptions, ETabState::OpenedTab)
		)
		->Split
		(
			// Main application area
			FTabManager::NewSplitter()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				// Main application area
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->SetHideTabWell(true)
					->SetSizeCoefficient(0.7f)
					->AddTab(WidgetReflectorTabID::WidgetHierarchy, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetHideTabWell(true)
					->SetSizeCoefficient(0.3f)
					->AddTab(WidgetReflectorTabID::SnapshotWidgetPicker, ETabState::ClosedTab)
					->AddTab(WidgetReflectorTabID::WidgetSearch, ETabState::ClosedTab)
#if WITH_SLATE_DEBUGGING
					->AddTab(WidgetReflectorTabID::WidgetEvents, ETabState::ClosedTab)
					->AddTab(WidgetReflectorTabID::HittestGrid, ETabState::ClosedTab)
#endif
					->AddTab(WidgetReflectorTabID::WidgetList, ETabState::ClosedTab)
					->AddTab(WidgetReflectorTabID::VisualElements, ETabState::ClosedTab)
				)
			)
#if WITH_EDITOR
			->Split
			(
				FTabManager::NewStack()
				->SetHideTabWell(true)
				->SetSizeCoefficient(0.3f)
				->AddTab(WidgetReflectorTabID::WidgetDetails, ETabState::ClosedTab)
			)
#endif
		)
	);

	auto RegisterTrackedTabSpawner = [this](const FName& TabId, const FOnSpawnTab& OnSpawnTab) -> FTabSpawnerEntry&
	{
		return TabManager->RegisterTabSpawner(TabId, FOnSpawnTab::CreateLambda([this, OnSpawnTab](const FSpawnTabArgs& Args) -> TSharedRef<SDockTab>
		{
			TSharedRef<SDockTab> SpawnedTab = OnSpawnTab.Execute(Args);
			OnTabSpawned(Args.GetTabId().TabType, SpawnedTab);
			return SpawnedTab;
		}));
	};

	check(InArgs._ParentTab.IsValid());
	TabManager = FGlobalTabmanager::Get()->NewTabManager(InArgs._ParentTab.ToSharedRef());
	TabManager->SetOnPersistLayout(FTabManager::FOnPersistLayout::CreateRaw(this, &SWidgetReflector::HandleTabManagerPersistLayout));

	RegisterTrackedTabSpawner(WidgetReflectorTabID::SlateOptions, FOnSpawnTab::CreateSP(this, &SWidgetReflector::SpawnSlateOptionWidgetTab))
		.SetDisplayName(LOCTEXT("OptionsTab", "Toolbar"));

#if WITH_EDITOR
	if (GIsEditor)
	{
		RegisterTrackedTabSpawner(WidgetReflectorTabID::WidgetDetails, FOnSpawnTab::CreateSP(this, &SWidgetReflector::SpawnWidgetDetails))
			.SetDisplayName(LOCTEXT("WidgetDetailsTab", "Widget Details"));
	}
#endif //WITH_EDITOR

#if WITH_SLATE_DEBUGGING
	RegisterTrackedTabSpawner(WidgetReflectorTabID::WidgetEvents, FOnSpawnTab::CreateSP(this, &SWidgetReflector::SpawnWidgetEvents))
		.SetDisplayName(LOCTEXT("WidgetEventsTab", "Widget Events"));
#endif // WITH_SLATE_DEBUGGING

	RegisterTrackedTabSpawner(WidgetReflectorTabID::WidgetHierarchy, FOnSpawnTab::CreateSP(this, &SWidgetReflector::SpawnWidgetHierarchyTab))
		.SetDisplayName(LOCTEXT("WidgetHierarchyTab", "Widget Hierarchy"));

	RegisterTrackedTabSpawner(WidgetReflectorTabID::WidgetList, FOnSpawnTab::CreateSP(this, &SWidgetReflector::SpawnWidgetList))
		.SetDisplayName(LOCTEXT("WidgetListTab", "Widget List"));

	RegisterTrackedTabSpawner(WidgetReflectorTabID::WidgetSearch, FOnSpawnTab::CreateSP(this, &SWidgetReflector::SpawnWidgetSearchTab))
		.SetDisplayName(LOCTEXT("WidgetSearchTab", "Widget Search"));

	RegisterTrackedTabSpawner(WidgetReflectorTabID::VisualElements, FOnSpawnTab::CreateSP(this, &SWidgetReflector::SpawnVisualElementsTab))
		.SetDisplayName(LOCTEXT("VisualElementsTab", "Elements"));

#if WITH_SLATE_DEBUGGING
	RegisterTrackedTabSpawner(WidgetReflectorTabID::HittestGrid, FOnSpawnTab::CreateSP(this, &SWidgetReflector::SpawnWidgetHittestGrid))
		.SetDisplayName(LOCTEXT("HitTestGridTab", "Hit Test Grid"));
#endif // WITH_SLATE_DEBUGGING

	RegisterTrackedTabSpawner(WidgetReflectorTabID::SnapshotWidgetPicker, FOnSpawnTab::CreateSP(this, &SWidgetReflector::SpawnSnapshotWidgetPicker))
		.SetDisplayName(LOCTEXT("SnapshotWidgetPickerTab", "Snapshot Widget Picker"));

#if WITH_EDITOR
	if (GIsEditor)
	{
		Layout = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, Layout);
	}
#endif

	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(TSharedPtr<FUICommandList>());
#if WITH_SLATE_DEBUGGING
	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("DemoModeLabel", "Demo Mode"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateRaw(FSlateReflectorModule::GetModulePtr()->GetInputEventVisualizer(), &FInputEventVisualizer::PopulateMenu),
		"DemoMode"
	);
#endif
	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("AtlasesMenuLabel", "Atlases"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateSP(this, &SWidgetReflector::HandlePullDownAtlasesMenu),
		"Atlases"
		);
	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("WindowMenuLabel", "Window"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateSP(this, &SWidgetReflector::HandlePullDownWindowMenu),
		"Window"
		);

	const TSharedRef<SWidget> MenuWidget = MenuBarBuilder.MakeWidget();
	TabManager->SetMenuMultiBox(MenuBarBuilder.GetMultiBox(), MenuWidget);

	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		.BorderBackgroundColor(FLinearColor::Gray) // Darken the outer border
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 4.0f, 0.0f, 0.0f))
			[
				MenuWidget
			]
			
			+ SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 4.0f, 0.0f, 0.0f))
			[
				TabManager->RestoreFrom(Layout, nullptr).ToSharedRef()
			]
		]
	];

#if WITH_EDITOR
	FEditorDelegates::EndPIE.AddSP(this, &SWidgetReflector::HandleEndPIE);
	FCoreUObjectDelegates::OnObjectTransacted.AddRaw(this, &SWidgetReflector::HandleObjectTransacted);
#endif //WITH_EDITOR
}

SWidgetReflector::~SWidgetReflector()
{
#if WITH_EDITOR
	FEditorDelegates::EndPIE.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);
#endif //WITH_EDITOR
}

void SWidgetReflector::HandlePullDownAtlasesMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("DisplayTextureAtlases", "Display Texture Atlases"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SWidgetReflector::HandleDisplayTextureAtlases)
		));
	MenuBuilder.AddMenuEntry(
		LOCTEXT("DisplayFontAtlases", "Display Font Atlases"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SWidgetReflector::HandleDisplayFontAtlases)
		));
}

void SWidgetReflector::HandlePullDownWindowMenu(FMenuBuilder& MenuBuilder)
{
	if (!TabManager.IsValid())
	{
		return;
	}

	TabManager->PopulateLocalTabSpawnerMenu(MenuBuilder);
}

TSharedRef<SDockTab> SWidgetReflector::SpawnSlateOptionWidgetTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("ToolbarTab", "Toolbar"))
		.ShouldAutosize(true)
		[
			SNew(SSlateOptions)
		];
	return SpawnedTab;
}

float SWidgetReflector::GetColumnWidth(FName InColumn, float InDefault) const
{
	if (const float* Width = ColumnWidths.Find(InColumn))
	{
		return FMath::Max(*Width, InDefault);
	}

	return InDefault;
}

void SWidgetReflector::UpdateColumnWidth(TSharedRef<FWidgetReflectorNodeBase> InItem, FName InColumn)
{
	if (ReflectorTree.IsValid())
	{
		if (TSharedPtr<SReflectorTreeWidgetItem> Row = StaticCastSharedPtr<SReflectorTreeWidgetItem>(ReflectorTree->WidgetFromItem(InItem)))
		{
			if (const TSharedRef<SWidget>* WidgetPtr = Row->GetWidgetFromColumnId(InColumn))
			{
				const float WidgetWidth = (*WidgetPtr)->GetDesiredSize().X;
				float& Width = ColumnWidths.FindOrAdd(InColumn);

				if (WidgetWidth > Width)
				{
					Width = WidgetWidth;

					if (ReflectorTree.IsValid())
					{
						ReflectorTree->GetHeaderRow()->SetColumnWidth(InColumn, WidgetWidth);
						ReflectorTree->GetHeaderRow()->OnColumnsChanged();
					}
				}
			}
		}
	}
}

TSharedRef<SDockTab> SWidgetReflector::SpawnWidgetHierarchyTab(const FSpawnTabArgs& Args)
{
	FSlimHorizontalToolBarBuilder ToolbarBuilderGlobal(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);

	TArray<FName> HiddenColumnsList;
	HiddenColumnsList.Reserve(HiddenReflectorTreeColumns.Num());
	for (const FString& Item : HiddenReflectorTreeColumns)
	{
		HiddenColumnsList.Add(*Item);
	}

	// Button that controls the target for the snapshot operation
	AvailableSnapshotTargetsComboBox = SNew(SComboBox<TSharedPtr<FWidgetSnapshotTarget>>)
	.IsEnabled(this, &SWidgetReflector::IsSnapshotTargetComboEnabled)
	.ToolTipText(LOCTEXT("ChooseSnapshotTargetToolTipText", "Choose Snapshot Target"))
	.OptionsSource(&AvailableSnapshotTargets)
	.OnGenerateWidget(this, &SWidgetReflector::HandleGenerateAvailableSnapshotComboItemWidget)
	.OnSelectionChanged(this, &SWidgetReflector::HandleAvailableSnapshotComboSelectionChanged)
	[
		SNew(STextBlock)
		.Text(this, &SWidgetReflector::GetSelectedSnapshotTargetDisplayName)
	];

	FSlateIcon EmptyIcon(FWidgetReflectorStyle::GetStyleSetName(), "Icon.Empty");
	ToolbarBuilderGlobal.BeginSection("Picking");
	{

		FTextBuilder TooltipText;
		ToolbarBuilderGlobal.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateSP(this, &SWidgetReflector::HandlePickingModeStateChanged),
				FCanExecuteAction::CreateSP(this,&SWidgetReflector::HandleHasPendingActions),
				FGetActionCheckState::CreateSP(this, &SWidgetReflector::HandleGetPickingButtonChecked)
			),
			NAME_None,
			MakeAttributeSP(this,&SWidgetReflector::HandleGetPickingModeText),
			TooltipText.ToText(),
			MakeAttributeSP(this, &SWidgetReflector::HandleGetPickingModeImage),
			EUserInterfaceActionType::ToggleButton
		);

		ToolbarBuilderGlobal.AddComboButton(
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction::CreateSP(this, &SWidgetReflector::HandleHasPendingActions),
				FGetActionCheckState()
			),
			FOnGetContent::CreateSP(this, &SWidgetReflector::HandlePickingModeContextMenu),
			FText::GetEmpty(),
			TooltipText.ToText(),
			EmptyIcon,
			true
		);


	}
	ToolbarBuilderGlobal.EndSection();

	ToolbarBuilderGlobal.BeginSection("Filter");
	{

		FTextBuilder TooltipText;

		ToolbarBuilderGlobal.AddComboButton(
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction::CreateSP(this, &SWidgetReflector::HandleHasPendingActions),
				FGetActionCheckState()
			),
			FOnGetContent::CreateSP(this, &SWidgetReflector::HandleReflectorTreeContextMenu),
			LOCTEXT("FilterLabel", "Filter"),
			TooltipText.ToText(),
			FSlateIcon(FWidgetReflectorStyle::GetStyleSetName(), WidgetReflectorIcon::Filter),
			false
		);

	}
	ToolbarBuilderGlobal.EndSection();

	ToolbarBuilderGlobal.AddWidget(SNew(SSpacer),NAME_None,true,HAlign_Right);
	ToolbarBuilderGlobal.BeginSection("Option");
	{

		FTextBuilder TooltipText;

		ToolbarBuilderGlobal.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateSP(this, &SWidgetReflector::HandleTakeSnapshotButtonClicked),
				FCanExecuteAction::CreateSP(this, &SWidgetReflector::IsTakeSnapshotButtonEnabled),
				FGetActionCheckState()
			),
			NAME_None,
			MakeAttributeSP(this, &SWidgetReflector::HandleGetTakeSnapshotText),
			TooltipText.ToText(),
			FSlateIcon(FWidgetReflectorStyle::GetStyleSetName(), WidgetReflectorIcon::TakeSnapshot),
			EUserInterfaceActionType::Button
		);

		ToolbarBuilderGlobal.AddComboButton(
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction::CreateSP(this, &SWidgetReflector::HandleHasPendingActions),
				FGetActionCheckState()
			),
			FOnGetContent::CreateSP(this, &SWidgetReflector::HandleSnapshotOptionsTreeContextMenu),
			LOCTEXT("OptionsLabel", "Options"),
			TooltipText.ToText(),
			FSlateIcon(FWidgetReflectorStyle::GetStyleSetName(), WidgetReflectorIcon::Ellipsis),
			false
		);

#if SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM

		ToolbarBuilderGlobal.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateSP(this, &SWidgetReflector::HandleLoadSnapshotButtonClicked),
				FCanExecuteAction::CreateSP(this, &SWidgetReflector::HandleHasPendingActions),
				FGetActionCheckState()
			),
			NAME_None,
			LOCTEXT("LoadSnapshotButtonText", "Load Snapshot"),
			TooltipText.ToText(),
			FSlateIcon(FWidgetReflectorStyle::GetStyleSetName(), WidgetReflectorIcon::LoadSnapshot),
			EUserInterfaceActionType::Button
		);

#endif
	}
	ToolbarBuilderGlobal.EndSection();
	ToolbarBuilderGlobal.SetStyle(&FAppStyle::Get(), "SlimToolBar");

	TSharedPtr<SScrollBar> VerticalScrollBar;
	TSharedPtr<SScrollBar> HorizontalScrollBar;

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)

		.Label(LOCTEXT("WidgetHierarchyTab", "Widget Hierarchy"))
		//.OnCanCloseTab_Lambda([]() { return false; }) // Can't prevent this as it stops the editor from being able to close while the widget reflector is open
		[
			SNew(SVerticalBox)
				
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 2.0f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					ToolbarBuilderGlobal.MakeWidget()
				]
			]
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(2.0f, 2.0f))
			[
				SNew(SVerticalBox)
				.Visibility(this, &SWidgetReflector::HandleIsBreadcrumbVisible)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSeparator).Thickness(5.f)
				]
				+ SVerticalBox::Slot()
				[
					SAssignNew(BreadCrumb, SBreadcrumbTrail<TSharedRef<FWidgetReflectorNodeBase>>)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.DelimiterImage(FAppStyle::Get().GetBrush("Icons.ChevronRight"))
					.TextStyle(FAppStyle::Get(), "NormalText")
					.OnCrumbClicked(this, &SWidgetReflector::HandleBreadcrumbOnClick)
					.GetCrumbMenuContent(this, &SWidgetReflector::HandleBreadcrumbDelimiterMenu)
				]
			]

			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SBorder)
				.Padding(0.f)
				.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SGridPanel)
					.FillColumn(0, 1.f)
					.FillRow(0, 1.f)

					+ SGridPanel::Slot(0, 1)
					[
						SAssignNew(HorizontalScrollBar, SScrollBar)
						.Orientation(Orient_Horizontal)
						.AlwaysShowScrollbar(false)
						.AlwaysShowScrollbarTrack(true)
					]

					+ SGridPanel::Slot(1, 0)
					[
						SAssignNew(VerticalScrollBar, SScrollBar)
						.Orientation(Orient_Vertical)
						.AlwaysShowScrollbar(false)
						.AlwaysShowScrollbarTrack(true)
					]
					
					+ SGridPanel::Slot(0, 0)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SNew(SScrollBox)
						.ExternalScrollbar(HorizontalScrollBar)
						.Orientation(Orient_Horizontal)
						.ScrollBarAlwaysVisible(true)
						
						+ SScrollBox::Slot()
						.AutoSize()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						[
							SNew(SScrollBox)
							.ExternalScrollbar(VerticalScrollBar)
							.Orientation(Orient_Vertical)
							.ScrollBarAlwaysVisible(true)

							+ SScrollBox::Slot()
							.AutoSize()
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Fill)
							[
								// The tree view that shows all the info that we capture.
								SAssignNew(ReflectorTree, SReflectorTree)
								.TreeItemsSource(&FilteredTreeRoot)
								.OnGenerateRow(this, &SWidgetReflector::HandleReflectorTreeGenerateRow)
								.OnGetChildren(this, &SWidgetReflector::HandleReflectorTreeGetChildren)
								.OnSelectionChanged(this, &SWidgetReflector::HandleReflectorTreeSelectionChanged)
								.OnContextMenuOpening(this, &SWidgetReflector::HandleReflectorTreeContextMenuPtr)
								.OnMouseButtonClick(this, &SWidgetReflector::HandleReflectorTreeOnMouseClick)
								.HighlightParentNodesForSelection(true)
								.ScrollBarStyle(&FWidgetReflectorStyle::Get().GetWidgetStyle<FScrollBarStyle>("ScrollBar.Hidden"))
								.HeaderRow
								(
									SNew(SHeaderRow)
									.CanSelectGeneratedColumn(true)
									.HiddenColumnsList(HiddenColumnsList)
									.OnHiddenColumnsListChanged(this, &SWidgetReflector::HandleReflectorTreeHiddenColumnsListChanged)

									+SHeaderRow::Column(SReflectorTreeWidgetItem::NAME_WidgetName)
									.DefaultLabel(LOCTEXT("WidgetName", "Widget Name"))								
									.ShouldGenerateWidget(true)
									.ManualWidth(this, &SWidgetReflector::GetColumnWidth, SReflectorTreeWidgetItem::NAME_WidgetName, 100.f)

									+SHeaderRow::Column(SReflectorTreeWidgetItem::NAME_ForegroundColor)
									.DefaultLabel(LOCTEXT("ForegroundColor", "FG"))
									.DefaultTooltip(LOCTEXT("ForegroundColorToolTip", "Foreground Color"))
									.FixedWidth(24.0f)

									+SHeaderRow::Column(SReflectorTreeWidgetItem::NAME_Visibility)
									.DefaultLabel(LOCTEXT("Visibility", "Visibility"))
									.DefaultTooltip(LOCTEXT("VisibilityTooltip", "Visibility"))
									.FixedWidth(125.0f)
						
									+ SHeaderRow::Column(SReflectorTreeWidgetItem::NAME_Enabled)
									.DefaultLabel(LOCTEXT("Enabled", "Enabled"))
									.DefaultTooltip(LOCTEXT("EnabledToolTip", "Enabled"))
									.FixedWidth(28.0f)

									+ SHeaderRow::Column(SReflectorTreeWidgetItem::NAME_Focusable)
									.DefaultLabel(LOCTEXT("Focus", "Focus"))
									.DefaultTooltip(LOCTEXT("FocusableTooltip", "Focusability (Note that for hit-test directional navigation to work it must be Focusable and \"Visible\"!)"))
									.FixedWidth(28.0f)

									+ SHeaderRow::Column(SReflectorTreeWidgetItem::NAME_HasActiveTimer)
									.DefaultLabel(LOCTEXT("HasActiveTimer", "Timer"))
									.DefaultTooltip(LOCTEXT("HasActiveTimerTooltip", "Has Active Timer"))
									.FixedWidth(28.0f)

									+SHeaderRow::Column(SReflectorTreeWidgetItem::NAME_Clipping)
									.DefaultLabel(LOCTEXT("Clipping", "Clipping" ))
									.FixedWidth(36.0f)

									+ SHeaderRow::Column(SReflectorTreeWidgetItem::NAME_LayerId)
									.DefaultLabel(LOCTEXT("LayerId", "LayerId"))
									.ManualWidth(this, &SWidgetReflector::GetColumnWidth, SReflectorTreeWidgetItem::NAME_LayerId, 100.f)

									+ SHeaderRow::Column(SReflectorTreeWidgetItem::NAME_ActualSize)
									.DefaultLabel(LOCTEXT("ActualSize", "Size"))
									.ManualWidth(this, &SWidgetReflector::GetColumnWidth, SReflectorTreeWidgetItem::NAME_ActualSize, 100.f)

									+SHeaderRow::Column(SReflectorTreeWidgetItem::NAME_WidgetInfo)
									.DefaultLabel(LOCTEXT("Source", "Source" ))
									.ManualWidth(this, &SWidgetReflector::GetColumnWidth, SReflectorTreeWidgetItem::NAME_WidgetInfo, 200.f)

									+SHeaderRow::Column(SReflectorTreeWidgetItem::NAME_Address)
									.DefaultLabel( LOCTEXT("Address", "Address") )
									.FixedWidth(150.f)
								)
							]
						]
					]
				]
			]
		];

	UpdateSelectedSnapshotTarget();

	return SpawnedTab;
}

TSharedRef<SDockTab> SWidgetReflector::SpawnSnapshotWidgetPicker(const FSpawnTabArgs& Args)
{
	TWeakPtr<SWidgetReflector> WeakSelf = SharedThis(this);
	auto OnTabClosed = [WeakSelf](TSharedRef<SDockTab>)
	{
		// Tab closed - leave snapshot mode
		if (TSharedPtr<SWidgetReflector> SelfPinned = WeakSelf.Pin())
		{
			SelfPinned->SetUIMode(EWidgetReflectorUIMode::Live);
		}
	};

	auto OnWidgetPathPicked = [WeakSelf](const TArray<TSharedRef<FWidgetReflectorNodeBase>>& InPickedWidgetPath)
	{
		if (TSharedPtr<SWidgetReflector> SelfPinned = WeakSelf.Pin())
		{
			SelfPinned->SelectedNodes.Reset();
			SelfPinned->PickedWidgetPath = InPickedWidgetPath;
			SelfPinned->UpdateFilteredTreeRoot();
		}
	};

	auto OnSnapshotWidgetPicked = [WeakSelf](FWidgetReflectorNodeBase::TPointerAsInt InSnapshotWidget)
	{
		if (TSharedPtr<SWidgetReflector> SelfPinned = WeakSelf.Pin())
		{
			SelfPinned->SelectedNodes.Reset();
			FWidgetReflectorNodeUtils::FindSnaphotWidget(SelfPinned->ReflectorTreeRoot, InSnapshotWidget, SelfPinned->PickedWidgetPath);
			SelfPinned->UpdateFilteredTreeRoot();
		}
	};

	return SNew(SDockTab)
		.Label(LOCTEXT("SnapshotWidgetPickerTab", "Snapshot Widget Picker"))
		.OnTabClosed_Lambda(OnTabClosed)
		[
			SAssignNew(WidgetSnapshotVisualizer, SWidgetSnapshotVisualizer)
			.SnapshotData(&SnapshotData)
			.OnWidgetPathPicked_Lambda(OnWidgetPathPicked)
			.OnSnapshotWidgetSelected_Lambda(OnSnapshotWidgetPicked)
		];
}

#if WITH_EDITOR

TSharedRef<SDockTab> SWidgetReflector::SpawnWidgetDetails(const FSpawnTabArgs& Args)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = true;
		DetailsViewArgs.bShowOptions = true;
		DetailsViewArgs.bAllowMultipleTopLevelObjects = false;
		DetailsViewArgs.bAllowFavoriteSystem = true;
		DetailsViewArgs.bShowObjectLabel = false;
		DetailsViewArgs.bHideSelectionTip = true;
	}
	TSharedRef<IDetailsView> PropertyView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	PropertyViewPtr = PropertyView;

	auto OnTabClosed = [this](TSharedRef<SDockTab>)
	{
	};

	return SNew(SDockTab)
		.Label(LOCTEXT("WidgetDetailsTab", "Widget Details"))
		.OnTabClosed_Lambda(OnTabClosed)
		[
			PropertyView
		];
}

#endif //WITH_EDITOR

#if WITH_SLATE_DEBUGGING

TSharedRef<SDockTab> SWidgetReflector::SpawnWidgetEvents(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("WidgetEventsTab", "Widget Events"))
		[
			SNew(SWidgetEventLog, AsShared())
			.OnWidgetTokenActivated(this, &SWidgetReflector::SelectLiveWidget)
		];
}

TSharedRef<SDockTab> SWidgetReflector::SpawnWidgetHittestGrid(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("HitTestGridTab", "Hit Test Grid"))
		[
			SAssignNew(WidgetHittestGrid, SWidgetHittestGrid, AsShared())
			.OnWidgetSelected(this, &SWidgetReflector::SelectLiveWidget)
			.OnVisualizeWidget(this, &SWidgetReflector::SetWidgetsToVisualize)
		];
}

#endif //WITH_SLATE_DEBUGGING

TSharedRef<SDockTab> SWidgetReflector::SpawnWidgetList(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("WidgetListTab", "Widget List"))
		[
			SNew(SWidgetList)
			.OnAccessSource(FAccessSourceCode::CreateSP(this, &SWidgetReflector::HandleSourceAccessDelegate))
			.OnAccessAsset(FAccessAsset::CreateSP(this, &SWidgetReflector::HandleAssetAccessDelegate))
		];
}

TSharedRef<SDockTab> SWidgetReflector::SpawnVisualElementsTab(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("DrawVisualElementsTab", "Draw Elements"))
		[
			SAssignNew(VisualElementsTabWidget, SVisualElementsTab, &VisualCapture)
			.OnWidgetSelected(this, &SWidgetReflector::SelectLiveWidget)
			.FilterWidgets(this, &SWidgetReflector::GetSelectedLiveWidgets)
			.IsPickingActive(this, &SWidgetReflector::IsDrawablePickingActive)
		];
}

TSharedRef<SDockTab> SWidgetReflector::SpawnWidgetSearchTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("WidgetSearchTab", "Widget Search"))
		[
			SNew(UE::SlateReflector::SWidgetSearchPanel)
			.OnSelectNode(this, &SWidgetReflector::SelectReflectorNode)
			.OnSelectLiveWidget(this, &SWidgetReflector::SelectLiveWidget)
			.OnGetRootNodes_Lambda([this]() -> TArray<TSharedRef<FWidgetReflectorNodeBase>>
			{
				return ReflectorTreeRoot;
			})
			.OnAccessSourceCode(FAccessSourceCode::CreateSP(this, &SWidgetReflector::HandleSourceAccessDelegate))
			.OnAccessAsset(FAccessAsset::CreateSP(this, &SWidgetReflector::HandleAssetAccessDelegate))
		];

	return SpawnedTab;
}

void SWidgetReflector::HandleTabManagerPersistLayout(const TSharedRef<FTabManager::FLayout>& LayoutToSave)
{
#if WITH_EDITOR
	FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, LayoutToSave);
#endif //WITH_EDITOR
}

void SWidgetReflector::SaveSettings()
{
	GConfig->SetArray(TEXT("WidgetReflector"), TEXT("HiddenReflectorTreeColumns"), HiddenReflectorTreeColumns, *GEditorPerProjectIni);
	GConfig->SetInt(TEXT("WidgetReflector"), TEXT("LastPickingMode"), static_cast<int32>(LastPickingMode), *GEditorPerProjectIni);
}


void SWidgetReflector::LoadSettings()
{
	if (GConfig->DoesSectionExist(TEXT("WidgetReflector"), *GEditorPerProjectIni))
	{
		int32 LastPickingModeAsInt = static_cast<int32>(EWidgetPickingMode::HitTesting);
		GConfig->GetInt(TEXT("WidgetReflector"), TEXT("LastPickingMode"), LastPickingModeAsInt, *GEditorPerProjectIni);
		LastPickingMode = ConvertToWidgetPickingMode(LastPickingModeAsInt);
		if (LastPickingMode == EWidgetPickingMode::None)
		{
			LastPickingMode = EWidgetPickingMode::HitTesting;
		}

		GConfig->GetArray(TEXT("WidgetReflector"), TEXT("HiddenReflectorTreeColumns"), HiddenReflectorTreeColumns, *GEditorPerProjectIni);
	}
}


void SWidgetReflector::OnTabSpawned(const FName& TabIdentifier, const TSharedRef<SDockTab>& SpawnedTab)
{
	TWeakPtr<SDockTab>* const ExistingTab = SpawnedTabs.Find(TabIdentifier);
	if (!ExistingTab)
	{
		SpawnedTabs.Add(TabIdentifier, SpawnedTab);
	}
	else
	{
		check(!ExistingTab->IsValid());
		*ExistingTab = SpawnedTab;
	}
}


void SWidgetReflector::CloseTab(const FName& TabIdentifier)
{
	TWeakPtr<SDockTab>* const ExistingTab = SpawnedTabs.Find(TabIdentifier);
	if (ExistingTab)
	{
		TSharedPtr<SDockTab> ExistingTabPin = ExistingTab->Pin();
		if (ExistingTabPin.IsValid())
		{
			ExistingTabPin->RequestCloseTab();
		}
	}
}

void SWidgetReflector::SetUIMode(const EWidgetReflectorUIMode InNewMode)
{
	if (CurrentUIMode != InNewMode)
	{
		CurrentUIMode = InNewMode;

		SelectedNodes.Reset();
		PickedWidgetPath.Reset();
		ReflectorTreeRoot.Reset();
		FilteredTreeRoot.Reset();
		ColumnWidths.Empty();
		ReflectorTree->RequestTreeRefresh();

		if (CurrentUIMode == EWidgetReflectorUIMode::Snapshot)
		{
			TabManager->TryInvokeTab(WidgetReflectorTabID::SnapshotWidgetPicker);
		}
		else
		{
			SnapshotData.ClearSnapshot();

			if (WidgetSnapshotVisualizer.IsValid())
			{
				WidgetSnapshotVisualizer->SnapshotDataUpdated();
			}

			CloseTab(WidgetReflectorTabID::SnapshotWidgetPicker);
		}
	}
	BreadCrumb->ClearCrumbs();
}


/* SCompoundWidget overrides
 *****************************************************************************/

void SWidgetReflector::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if (bIsPendingDelayedSnapshot && FSlateApplication::Get().GetCurrentTime() > TimeOfScheduledSnapshot)
	{
		// TakeSnapshot leads to the widget being ticked indirectly recursively,
		// so the recursion of this tick mustn't trigger a recursive snapshot.
		// Immediately clear the pending snapshot flag.
		bIsPendingDelayedSnapshot = false;
		TimeOfScheduledSnapshot = -1.0;

		TakeSnapshot();
	}

	//if (VisualCapture.IsEnabled() && VisualElementsTabWidget.IsValid())
	//{
	//	VisualElementsTabWidget->SetDrawablePickingActive(PickingMode == EWidgetPickingMode::Drawable);
	//	VisualElementsTabWidget->Refresh();
	//}
}


/* IWidgetReflector overrides
 *****************************************************************************/

bool SWidgetReflector::ReflectorNeedsToDrawIn( TSharedRef<SWindow> ThisWindow ) const
{
	return ((SelectedNodes.Num() > 0) && (ReflectorTreeRoot.Num() > 0) && (ReflectorTreeRoot[0]->GetLiveWidget() == ThisWindow));
}

void SWidgetReflector::SetWidgetsToVisualize( const FWidgetPath& InWidgetsToVisualize )
{
	ReflectorTreeRoot.Reset();
	FilteredTreeRoot.Reset();
	PickedWidgetPath.Reset();
	SelectedNodes.Reset();
	ColumnWidths.Empty();

	if (InWidgetsToVisualize.IsValid())
	{
		ReflectorTreeRoot.Add(FWidgetReflectorNodeUtils::NewLiveNodeTreeFrom(InWidgetsToVisualize.Widgets[0]));
		FWidgetReflectorNodeUtils::FindLiveWidgetPath(ReflectorTreeRoot, InWidgetsToVisualize, PickedWidgetPath);
		UpdateFilteredTreeRoot();
	}

	ReflectorTree->RequestTreeRefresh();
}

int32 SWidgetReflector::Visualize( const FWidgetPath& InWidgetsToVisualize, FSlateWindowElementList& OutDrawElements, int32 LayerId )
{
	if (!InWidgetsToVisualize.IsValid() && SelectedNodes.Num() > 0 && ReflectorTreeRoot.Num() > 0)
	{
		TSharedPtr<SWidget> WindowWidget = ReflectorTreeRoot[0]->GetLiveWidget();
		if (WindowWidget.IsValid())
		{
			TSharedPtr<SWindow> Window = StaticCastSharedPtr<SWindow>(WindowWidget);
			return VisualizeSelectedNodesAsRectangles(SelectedNodes, Window.ToSharedRef(), OutDrawElements, LayerId);
		}
	}

	const bool bAttemptingToVisualizeReflector = InWidgetsToVisualize.ContainsWidget(ReflectorTree.Get());

	if (PickingMode == EWidgetPickingMode::Drawable)
	{
		TSharedPtr<const FVisualTreeSnapshot> Tree = VisualCapture.GetVisualTreeForWindow(OutDrawElements.GetPaintWindow());
		if (Tree)
		{
			const FVector2f AbsPoint = FSlateApplication::Get().GetCursorPos();
			const FVector2f WindowPoint = AbsPoint - OutDrawElements.GetPaintWindow()->GetPositionInScreen();
			if (TSharedPtr<const SWidget> PickedWidget = Tree->Pick(WindowPoint))
			{
				FWidgetPath WidgetsToVisualize = InWidgetsToVisualize;
				FSlateApplication::Get().FindPathToWidget(PickedWidget.ToSharedRef(), WidgetsToVisualize, EVisibility::All);
				if (!bAttemptingToVisualizeReflector)
				{
					SetWidgetsToVisualize(WidgetsToVisualize);
					return VisualizePickAsRectangles(WidgetsToVisualize, OutDrawElements, LayerId);
				}
			}
			else
			{
				SetWidgetsToVisualize(FWidgetPath{});
			}
		}
	}
	else if (!bAttemptingToVisualizeReflector)
	{
		SetWidgetsToVisualize(InWidgetsToVisualize);
		return VisualizePickAsRectangles(InWidgetsToVisualize, OutDrawElements, LayerId);
	}

	return LayerId;
}

/* SWidgetReflector implementation
 *****************************************************************************/

void SWidgetReflector::SelectLiveWidget(TSharedPtr<const SWidget> InWidget)
{
	bool bFound = false;
	if (this->CurrentUIMode == EWidgetReflectorUIMode::Live && InWidget)
	{
		TArray<TSharedRef<FWidgetReflectorNodeBase>> FoundList;
		FWidgetReflectorNodeUtils::FindLiveWidget(ReflectorTreeRoot, InWidget, FoundList);
		if (FoundList.Num() > 0)
		{
			for (const TSharedRef<FWidgetReflectorNodeBase>& FoundItem : FoundList)
			{
				ReflectorTree->SetItemExpansion(FoundItem, true);
			}
			ReflectorTree->RequestScrollIntoView(FoundList.Last());
			ReflectorTree->SetSelection(FoundList.Last());
			bFound = true;
		}
	}

	if (!bFound)
	{
		ReflectorTree->ClearSelection();
	}

	if (bFound && TabManager.IsValid())
	{
		TabManager->TryInvokeTab(WidgetReflectorTabID::WidgetHierarchy);
	}
}

void SWidgetReflector::SelectReflectorNode(TSharedRef<FWidgetReflectorNodeBase> InNode)
{
	if (!ReflectorTree.IsValid())
	{
		return;
	}

	ReflectorTree->ClearSelection();

	// Walk up the parent chain and expand each ancestor so the node is visible in the tree.
	TArray<TSharedRef<FWidgetReflectorNodeBase>> PathToExpand;
	TSharedPtr<FWidgetReflectorNodeBase> Parent = InNode->GetParentNode();
	while (Parent.IsValid())
	{
		PathToExpand.Add(Parent.ToSharedRef());
		Parent = Parent->GetParentNode();
	}

	for (int32 Index = PathToExpand.Num() - 1; Index >= 0; --Index)
	{
		const TSharedRef<FWidgetReflectorNodeBase>& PathNode = PathToExpand[Index];
		ReflectorTree->SetItemExpansion(PathNode, true);
	}

	ReflectorTree->SetSelection(InNode);
	ReflectorTree->RequestScrollIntoView(InNode);
}

TArray<TSharedPtr<const SWidget>> SWidgetReflector::GetSelectedLiveWidgets() const
{
	TArray<TSharedPtr<const SWidget>> Result;
	if (ReflectorTree)
	{
		TArray<TSharedRef<FWidgetReflectorNodeBase>> SelectedItems;
		ReflectorTree->GetSelectedItems(SelectedItems);
		for (const TSharedRef<FWidgetReflectorNodeBase>& Item : SelectedItems)
		{
			if (TSharedPtr<const SWidget> Widget = Item->GetLiveWidget())
			{
				Result.Add(Widget);
			}
		}
	}
	return Result;
}

namespace WidgetReflectorRecursive
{
	bool FindNodeWithReflectionData(const TArray<TSharedRef<FWidgetReflectorNodeBase>>& NodeBase, TArray<TSharedRef<FWidgetReflectorNodeBase>>& Result)
	{
		for (const TSharedRef<FWidgetReflectorNodeBase>& Node : NodeBase)
		{
			if (Node->HasValidWidgetAssetData())
			{
				return true;
			}

		}
		for (const TSharedRef<FWidgetReflectorNodeBase>& Node : NodeBase)
		{
			if (FindNodeWithReflectionData(Node->GetChildNodes(), Result))
			{
				Result.Add(Node);
			}
		}
		return false;
	}
}

void SWidgetReflector::UpdateFilteredTreeRoot()
{
	FilteredTreeRoot.Reset();
	if (bFilterReflectorTreeRootWithUMG)
	{
		WidgetReflectorRecursive::FindNodeWithReflectionData(ReflectorTreeRoot, FilteredTreeRoot);
		VisualizeAsTree(PickedWidgetPath);
	}
	else
	{
		FilteredTreeRoot = ReflectorTreeRoot;
		VisualizeAsTree(PickedWidgetPath);
	}
}

void SWidgetReflector::SetNodesAsReflectorTreeRoot(TArray<TSharedRef<FWidgetReflectorNodeBase>> RootNodes)
{
	TArray<TSharedRef<FWidgetReflectorNodeBase>> FilteredNodes = FilterSelectedToSetAsReflectorTreeRoot(RootNodes);
	if (FilteredNodes.Num() > 0)
	{
		FilteredTreeRoot.Reset();
		FilteredTreeRoot.Append(FilteredNodes);
		ColumnWidths.Empty();
		ReflectorTree->RequestTreeRefresh();
		TSharedPtr<FWidgetReflectorNodeBase> FirstNodeParent = FilteredNodes[0]->GetParentNode();
		if (FirstNodeParent.IsValid())
		{
			CreateCrumbTrailForNode(FirstNodeParent.ToSharedRef());
		}
		else
		{
			BreadCrumb->ClearCrumbs();
		}
	}
}

TArray<TSharedRef<FWidgetReflectorNodeBase>> SWidgetReflector::FilterSelectedToSetAsReflectorTreeRoot(TArray<TSharedRef<FWidgetReflectorNodeBase>> RootNodes)
{
	if (RootNodes.Num() > 1)
	{
		TArray<TSharedRef<FWidgetReflectorNodeBase>> ShallowestNodes;
		ShallowestNodes = RootNodes;
		for (int32 index = ShallowestNodes.Num() -1 ; index >= 0; index--)
		{
			if (ShallowestNodes.Contains(ShallowestNodes[index]->GetParentNode()))
			{
				ShallowestNodes.RemoveAt(index);
			}
		}
		TSharedPtr<FWidgetReflectorNodeBase> FirstNodeParent = ShallowestNodes[0]->GetParentNode();
		for (int32 index = ShallowestNodes.Num() - 1; index >= 0; index--)
		{
			if (ShallowestNodes[index]->GetParentNode() != FirstNodeParent)
			{
				ShallowestNodes.RemoveAt(index);
			}
		}
		return ShallowestNodes;
	}
	return RootNodes;
}
TSharedPtr<IToolTip> SWidgetReflector::GenerateToolTipForReflectorNode( TWeakPtr<FWidgetReflectorNodeBase> InReflectorNode ) const
{
	if (TSharedPtr<FWidgetReflectorNodeBase> ReflectorNode = InReflectorNode.Pin())
	{
		return SNew(SToolTip)
			[
				SNew(SReflectorToolTipWidget)
				.WidgetInfoToVisualize(ReflectorNode)
			];
	}
	return FSlateApplication::Get().MakeToolTip(LOCTEXT("MissingNode", "The node is invalid."));
}

void SWidgetReflector::VisualizeAsTree( const TArray<TSharedRef<FWidgetReflectorNodeBase>>& WidgetPathToVisualize )
{
	if (WidgetPathToVisualize.Num() > 0)
	{
		const FLinearColor TopmostWidgetColor(1.0f, 0.0f, 0.0f);
		const FLinearColor LeafmostWidgetColor(0.0f, 1.0f, 0.0f);

		for (int32 WidgetIndex = 0; WidgetIndex < WidgetPathToVisualize.Num(); ++WidgetIndex)
		{
			const auto& CurWidget = WidgetPathToVisualize[WidgetIndex];

			// Tint the item based on depth in picked path
			const float ColorFactor = static_cast<float>(WidgetIndex) / static_cast<float>(WidgetPathToVisualize.Num());
			CurWidget->SetTint(FMath::Lerp(TopmostWidgetColor, LeafmostWidgetColor, ColorFactor));

			// Make sure the user can see the picked path in the tree.
			ReflectorTree->SetItemExpansion(CurWidget, true);
		}

		ReflectorTree->RequestScrollIntoView(WidgetPathToVisualize.Last());
		ReflectorTree->SetSelection(WidgetPathToVisualize.Last());
	}
	else
	{
		ReflectorTree->ClearSelection();
	}
}


int32 SWidgetReflector::VisualizePickAsRectangles( const FWidgetPath& InWidgetsToVisualize, FSlateWindowElementList& OutDrawElements, int32 LayerId)
{
	const FLinearColor TopmostWidgetColor(1.0f, 0.0f, 0.0f);
	const FLinearColor LeafmostWidgetColor(0.0f, 1.0f, 0.0f);

	for (int32 WidgetIndex = 0; WidgetIndex < InWidgetsToVisualize.Widgets.Num(); ++WidgetIndex)
	{
		const FArrangedWidget& WidgetGeometry = InWidgetsToVisualize.Widgets[WidgetIndex];
		const float ColorFactor = static_cast<float>(WidgetIndex)/ static_cast<float>(InWidgetsToVisualize.Widgets.Num());
		const FLinearColor Tint(1.0f - ColorFactor, ColorFactor, 0.0f, 1.0f);

		// The FGeometry we get is from a WidgetPath, so it's rooted in desktop space.
		// We need to APPEND a transform to the Geometry to essentially undo this root transform
		// and get us back into Window Space.
		// This is nonstandard so we have to go through some hoops and a specially exposed method 
		// in FPaintGeometry to allow appending layout transforms.
		FPaintGeometry WindowSpaceGeometry = WidgetGeometry.Geometry.ToPaintGeometry();
		WindowSpaceGeometry.AppendTransform(TransformCast<FSlateLayoutTransform>(Inverse(InWidgetsToVisualize.TopLevelWindow->GetPositionInScreen())));

		FLinearColor Color = FMath::Lerp(TopmostWidgetColor, LeafmostWidgetColor, ColorFactor);
		DrawWidgetVisualization(WindowSpaceGeometry, Color, OutDrawElements, LayerId);
	}

	return LayerId;
}

int32 SWidgetReflector::VisualizeSelectedNodesAsRectangles( const TArray<TSharedRef<FWidgetReflectorNodeBase>>& InNodesToDraw, const TSharedRef<SWindow>& VisualizeInWindow, FSlateWindowElementList& OutDrawElements, int32 LayerId )
{
	for (int32 NodeIndex = 0; NodeIndex < InNodesToDraw.Num(); ++NodeIndex)
	{
		const TSharedRef<FWidgetReflectorNodeBase>& NodeToDraw = InNodesToDraw[NodeIndex];
		const FLinearColor Tint(0.0f, 1.0f, 0.0f);

		// The FGeometry we get is from a WidgetPath, so it's rooted in desktop space.
		// We need to APPEND a transform to the Geometry to essentially undo this root transform
		// and get us back into Window Space.
		// This is nonstandard so we have to go through some hoops and a specially exposed method 
		// in FPaintGeometry to allow appending layout transforms.
		FPaintGeometry WindowSpaceGeometry(NodeToDraw->GetAccumulatedLayoutTransform(), NodeToDraw->GetAccumulatedRenderTransform(), NodeToDraw->GetLocalSize(), NodeToDraw->GetGeometry().HasRenderTransform());
		WindowSpaceGeometry.AppendTransform(TransformCast<FSlateLayoutTransform>(Inverse(VisualizeInWindow->GetPositionInScreen())));

		DrawWidgetVisualization(WindowSpaceGeometry, NodeToDraw->GetTint(), OutDrawElements, LayerId);
	}

	return LayerId;
}

void SWidgetReflector::DrawWidgetVisualization(const FPaintGeometry& WidgetGeometry, FLinearColor Color, FSlateWindowElementList& OutDrawElements, int32& LayerId)
{
	WidgetGeometry.CommitTransformsIfUsingLegacyConstructor();
	const FVector2D LocalSize = WidgetGeometry.GetLocalSize();

	// If the size is 0 in any dimension, we're going to draw a line to represent the widget, since it's going to take up
	// padding space since it's visible, even though it's zero sized.
	if (FMath::IsNearlyZero(LocalSize.X) || FMath::IsNearlyZero(LocalSize.Y))
	{
		TArray<FVector2D> LinePoints;
		LinePoints.SetNum(2);

		LinePoints[0] = FVector2D::ZeroVector;
		LinePoints[1] = LocalSize;

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			++LayerId,
			WidgetGeometry,
			LinePoints,
			ESlateDrawEffect::None,
			Color,
			true,
			2
		);
	}
	else
	{
		// Draw a normal box border around the geometry
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			WidgetGeometry,
			FCoreStyle::Get().GetBrush(TEXT("Debug.Border")),
			ESlateDrawEffect::None,
			Color
		);
	}
}


/* SWidgetReflector callbacks
 *****************************************************************************/

void SWidgetReflector::HandleDisplayTextureAtlases()
{
	static const FName SlateReflectorModuleName("SlateReflector");
	FModuleManager::LoadModuleChecked<ISlateReflectorModule>(SlateReflectorModuleName).DisplayTextureAtlasVisualizer();
}

void SWidgetReflector::HandleDisplayFontAtlases()
{
	static const FName SlateReflectorModuleName("SlateReflector");
	FModuleManager::LoadModuleChecked<ISlateReflectorModule>(SlateReflectorModuleName).DisplayFontAtlasVisualizer();
}


/* Picking button
 *****************************************************************************/

ECheckBoxState SWidgetReflector::HandleGetPickingButtonChecked() const
{
	return PickingMode != EWidgetPickingMode::None ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SWidgetReflector::HandlePickingModeStateChanged()
{
	if (PickingMode == EWidgetPickingMode::None)
	{
		SetPickingMode(LastPickingMode);
	}
	else
	{
		SetPickingMode(EWidgetPickingMode::None);
	}

	if (IsVisualizingLayoutUnderCursor())
	{
		SetUIMode(EWidgetReflectorUIMode::Live);
	}
}

 FSlateIcon SWidgetReflector::HandleGetPickingModeImage() const
{


	switch (LastPickingMode)
	{
	case EWidgetPickingMode::Focus:
		return FSlateIcon(FWidgetReflectorStyle::GetStyleSetName(), WidgetReflectorIcon::FocusPicking);
	case EWidgetPickingMode::HitTesting:
		return FSlateIcon(FWidgetReflectorStyle::GetStyleSetName(), WidgetReflectorIcon::HitTestPicking);
	case EWidgetPickingMode::Drawable:
		return FSlateIcon(FWidgetReflectorStyle::GetStyleSetName(), WidgetReflectorIcon::VisualPicking);
	case EWidgetPickingMode::None:
	default:
		return FSlateIcon(FWidgetReflectorStyle::GetStyleSetName(), "Icon.Empty");
	}
}

FText SWidgetReflector::HandleGetPickingModeText() const
{
	if (PickingMode == EWidgetPickingMode::None)
	{
		switch(LastPickingMode)
		{
		case EWidgetPickingMode::Focus:
			return WidgetReflectorText::Focus;
		case EWidgetPickingMode::Drawable:
			return WidgetReflectorText::VisualPicking;
		case EWidgetPickingMode::HitTesting:
			return WidgetReflectorText::HitTestPicking;
		}
	}
	else if (PickingMode == EWidgetPickingMode::Focus)
	{
		return WidgetReflectorText::Focusing;
	}
	return WidgetReflectorText::Picking;
}

TSharedRef<SWidget> SWidgetReflector::HandlePickingModeContextMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	const bool bIsFocus = PickingMode == EWidgetPickingMode::Focus;
	MenuBuilder.AddMenuEntry(
		WidgetReflectorText::Focus,
		FText::GetEmpty(),
		FSlateIcon(FWidgetReflectorStyle::GetStyleSetName(), WidgetReflectorIcon::FocusPicking),
		FUIAction(
			FExecuteAction::CreateSP(this, &SWidgetReflector::HandlePickButtonClicked, EWidgetPickingMode::Focus),
			FCanExecuteAction::CreateLambda([bIsFocus](){ return !bIsFocus; })
		));

	const bool bIsHitTestPicking = PickingMode == EWidgetPickingMode::HitTesting;
	MenuBuilder.AddMenuEntry(
		WidgetReflectorText::HitTestPicking,
		FText::GetEmpty(),
		FSlateIcon(FWidgetReflectorStyle::GetStyleSetName(), WidgetReflectorIcon::HitTestPicking),
		FUIAction(
			FExecuteAction::CreateSP(this, &SWidgetReflector::HandlePickButtonClicked, EWidgetPickingMode::HitTesting),
			FCanExecuteAction::CreateLambda([bIsHitTestPicking]() { return !bIsHitTestPicking; })
		));

	const bool bIsDrawable = PickingMode == EWidgetPickingMode::Drawable;
	MenuBuilder.AddMenuEntry(
		WidgetReflectorText::VisualPicking,
		FText::GetEmpty(),
		FSlateIcon(FWidgetReflectorStyle::GetStyleSetName(), WidgetReflectorIcon::VisualPicking),
		FUIAction(
			FExecuteAction::CreateSP(this, &SWidgetReflector::HandlePickButtonClicked, EWidgetPickingMode::Drawable),
			FCanExecuteAction::CreateLambda([bIsDrawable]() { return !bIsDrawable; })
		));

	return MenuBuilder.MakeWidget();
}

void SWidgetReflector::HandlePickButtonClicked(EWidgetPickingMode InPickingMode)
{
	bool bHasChanged = LastPickingMode != InPickingMode;
	LastPickingMode = InPickingMode;
	SetPickingMode(PickingMode != InPickingMode ? InPickingMode : EWidgetPickingMode::None);

	if (IsVisualizingLayoutUnderCursor())
	{
		SetUIMode(EWidgetReflectorUIMode::Live);
	}

	if (bHasChanged)
	{
		SaveSettings();
	}
}

void SWidgetReflector::EnableVisualCapture(FName Enabler)
{
	VisualCaptureEnablers.AddUnique(Enabler);
	if (!VisualCapture.IsEnabled())
	{
		VisualCapture.Enable();
	}
}

void SWidgetReflector::DisableVisualCapture(FName Enabler)
{
	VisualCaptureEnablers.RemoveSingleSwap(Enabler);
	if (VisualCaptureEnablers.Num() == 0 && VisualCapture.IsEnabled())
	{
		VisualCapture.Disable();
	}
}

bool SWidgetReflector::IsSnapshotTargetComboEnabled() const
{
	if (bIsPendingDelayedSnapshot)
	{
		return false;
	}

#if SLATE_REFLECTOR_HAS_SESSION_SERVICES
	return !RemoteSnapshotRequestId.IsValid();
#else
	return false;
#endif
}

bool SWidgetReflector::IsTakeSnapshotButtonEnabled() const
{
	return SelectedSnapshotTargetInstanceId.IsValid() && !RemoteSnapshotRequestId.IsValid();
}

void SWidgetReflector::HandleTakeSnapshotButtonClicked()
{
	if (!bIsPendingDelayedSnapshot)
	{
		if (SnapshotDelay > 0.0f)
		{
			bIsPendingDelayedSnapshot = true;
			TimeOfScheduledSnapshot = FSlateApplication::Get().GetCurrentTime() + SnapshotDelay;
		}
		else
		{
			TakeSnapshot();
		}
	}
	else
	{
		bIsPendingDelayedSnapshot = false;
		TimeOfScheduledSnapshot = -1.0f;
	}

}

TSharedRef<SWidget> SWidgetReflector::HandleSnapshotOptionsTreeContextMenu()
{
	TSharedRef<SWidget> DelayWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("DelayLabel", "Delay"))
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		[
			SNew(SSpinBox<float>)
			.MinValue(0.f)
			.MinDesiredWidth(40.f)
			.Value_Lambda([this]() { return SnapshotDelay; })
			.OnValueCommitted_Lambda([this](const float InValue, ETextCommit::Type) { SnapshotDelay = FMath::Max(0.0f, InValue); })
		];

	TSharedRef<SWidget> NavigationEventSimulationWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("NavigationEventSimulationLabel", "Navigation Event Simulation"))
			.ToolTipText(LOCTEXT("NavigationEventSimulationTooltip", "Build a simulation of all the possible Navigation Events that can occur in the windows."))
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(4.f, 0.f))
		.HAlign(HAlign_Right)
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([this]() { return bRequestNavigationSimulation ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState) { bRequestNavigationSimulation = NewState == ECheckBoxState::Checked; })
		];

	return SNew(SVerticalBox)
	+ SVerticalBox::Slot()
	.Padding(2.f)
	[
		DelayWidget
	]
	+ SVerticalBox::Slot()
	.Padding(2.f)
	[
		NavigationEventSimulationWidget
	]
	+ SVerticalBox::Slot()
	.Padding(2.f)
	[
		AvailableSnapshotTargetsComboBox.ToSharedRef()
	];
}

void SWidgetReflector::TakeSnapshot()
{
	// Local snapshot?
	if (SelectedSnapshotTargetInstanceId == FApp::GetInstanceId())
	{
		SetUIMode(EWidgetReflectorUIMode::Snapshot);

#if WITH_SLATE_DEBUGGING
		if (TSharedPtr<SWidgetHittestGrid> WidgetHittestGridPin = WidgetHittestGrid.Pin())
		{
			WidgetHittestGridPin->SetPause(true);
		}
#endif

		// Take a snapshot of any window(s) that are currently open
		SnapshotData.TakeSnapshot(bRequestNavigationSimulation);

		// Rebuild the reflector tree from the snapshot data
		VisualCapture.Reset();
		SelectedNodes.Reset();
		PickedWidgetPath.Reset();
		ReflectorTreeRoot = FilteredTreeRoot = SnapshotData.GetWindowsRef();
		ColumnWidths.Empty();
		ReflectorTree->RequestTreeRefresh();

		WidgetSnapshotVisualizer->SnapshotDataUpdated();

#if WITH_SLATE_DEBUGGING
		if (TSharedPtr<SWidgetHittestGrid> WidgetHittestGridPin = WidgetHittestGrid.Pin())
		{
			WidgetHittestGridPin->SetPause(false);
		}
#endif
	}
	else
	{
		// Remote snapshot - these can take a while, show a progress message
		FNotificationInfo Info(LOCTEXT("RemoteWidgetSnapshotPendingNotificationText", "Waiting for Remote Widget Snapshot Data"));

		// Add the buttons with text, tooltip and callback
		Info.ButtonDetails.Add(FNotificationButtonInfo(
			LOCTEXT("CancelPendingSnapshotButtonText", "Cancel"),
			LOCTEXT("CancelPendingSnapshotButtonToolTipText", "Cancel the pending widget snapshot request."),
			FSimpleDelegate::CreateSP(this, &SWidgetReflector::OnCancelPendingRemoteSnapshot)
		));

		// We will be keeping track of this ourselves
		Info.bFireAndForget = false;

		// Launch notification
		WidgetSnapshotNotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);

		if (WidgetSnapshotNotificationPtr.IsValid())
		{
			WidgetSnapshotNotificationPtr.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
		}

		RemoteSnapshotRequestId = WidgetSnapshotService->RequestSnapshot(SelectedSnapshotTargetInstanceId, FWidgetSnapshotService::FOnWidgetSnapshotResponse::CreateSP(this, &SWidgetReflector::HandleRemoteSnapshotReceived));

		if (!RemoteSnapshotRequestId.IsValid())
		{
			TSharedPtr<SNotificationItem> WidgetSnapshotNotificationPin = WidgetSnapshotNotificationPtr.Pin();

			if (WidgetSnapshotNotificationPin.IsValid())
			{
				WidgetSnapshotNotificationPin->SetText(LOCTEXT("RemoteWidgetSnapshotFailedNotificationText", "Remote Widget Snapshot Failed"));
				WidgetSnapshotNotificationPin->SetCompletionState(SNotificationItem::CS_Fail);
				WidgetSnapshotNotificationPin->ExpireAndFadeout();

				WidgetSnapshotNotificationPtr.Reset();
			}
		}
	}
}

void SWidgetReflector::OnCancelPendingRemoteSnapshot()
{
	TSharedPtr<SNotificationItem> WidgetSnapshotNotificationPin = WidgetSnapshotNotificationPtr.Pin();

	if (WidgetSnapshotNotificationPin.IsValid())
	{
		WidgetSnapshotNotificationPin->SetText(LOCTEXT("RemoteWidgetSnapshotAbortedNotificationText", "Aborted Remote Widget Snapshot"));
		WidgetSnapshotNotificationPin->SetCompletionState(SNotificationItem::CS_Fail);
		WidgetSnapshotNotificationPin->ExpireAndFadeout();

		WidgetSnapshotNotificationPtr.Reset();
	}

	WidgetSnapshotService->AbortSnapshotRequest(RemoteSnapshotRequestId);
	RemoteSnapshotRequestId = FGuid();
}

void SWidgetReflector::HandleRemoteSnapshotReceived(const TArray<uint8>& InSnapshotData)
{
	{
		TSharedPtr<SNotificationItem> WidgetSnapshotNotificationPin = WidgetSnapshotNotificationPtr.Pin();

		if (WidgetSnapshotNotificationPin.IsValid())
		{
			WidgetSnapshotNotificationPin->SetText(LOCTEXT("RemoteWidgetSnapshotReceivedNotificationText", "Remote Widget Snapshot Data Received"));
			WidgetSnapshotNotificationPin->SetCompletionState(SNotificationItem::CS_Success);
			WidgetSnapshotNotificationPin->ExpireAndFadeout();

			WidgetSnapshotNotificationPtr.Reset();
		}
	}

	RemoteSnapshotRequestId = FGuid();

	SetUIMode(EWidgetReflectorUIMode::Snapshot);

	// Load up the remote data
	SnapshotData.LoadSnapshotFromBuffer(InSnapshotData);

	// Rebuild the reflector tree from the snapshot data
	SelectedNodes.Reset();
	PickedWidgetPath.Reset();
	ReflectorTreeRoot = FilteredTreeRoot = SnapshotData.GetWindowsRef();
	ColumnWidths.Empty();
	ReflectorTree->RequestTreeRefresh();

	WidgetSnapshotVisualizer->SnapshotDataUpdated();
}

#if SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM

void SWidgetReflector::HandleLoadSnapshotButtonClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

	if (DesktopPlatform)
	{
		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this));

		TArray<FString> OpenFilenames;
		const bool bOpened = DesktopPlatform->OpenFileDialog(
			(ParentWindow.IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr,
			LOCTEXT("LoadSnapshotDialogTitle", "Load Widget Snapshot").ToString(),
			FPaths::GameAgnosticSavedDir(),
			TEXT(""),
			TEXT("Slate Widget Snapshot (*.widgetsnapshot)|*.widgetsnapshot"),
			EFileDialogFlags::None,
			OpenFilenames
			);

		if (bOpened && SnapshotData.LoadSnapshotFromFile(OpenFilenames[0]))
		{
			SetUIMode(EWidgetReflectorUIMode::Snapshot);

			// Rebuild the reflector tree from the snapshot data
			ReflectorTreeRoot = SnapshotData.GetWindowsRef();
			ColumnWidths.Empty();
			ReflectorTree->RequestTreeRefresh();

			WidgetSnapshotVisualizer->SnapshotDataUpdated();
		}
	}
}

#endif // SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM

void SWidgetReflector::UpdateAvailableSnapshotTargets()
{
	AvailableSnapshotTargets.Reset();

#if SLATE_REFLECTOR_HAS_SESSION_SERVICES
	{
		TSharedPtr<ISessionManager> SessionManager = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices").GetSessionManager();
		if (SessionManager.IsValid())
		{
			TArray<TSharedPtr<ISessionInfo>> AvailableSessions;
			SessionManager->GetSessions(AvailableSessions);

			for (const auto& AvailableSession : AvailableSessions)
			{
				// Only allow sessions belonging to the current user
				if (AvailableSession->GetSessionOwner() != FApp::GetSessionOwner())
				{
					continue;
				}

				TArray<TSharedPtr<ISessionInstanceInfo>> AvailableInstances;
				AvailableSession->GetInstances(AvailableInstances);

				for (const auto& AvailableInstance : AvailableInstances)
				{
					FWidgetSnapshotTarget SnapshotTarget;
					SnapshotTarget.DisplayName = FText::Format(LOCTEXT("SnapshotTargetDisplayNameFmt", "{0} ({1})"), FText::FromString(AvailableInstance->GetInstanceName()), FText::FromString(AvailableInstance->GetPlatformName()));
					SnapshotTarget.InstanceId = AvailableInstance->GetInstanceId();

					AvailableSnapshotTargets.Add(MakeShareable(new FWidgetSnapshotTarget(SnapshotTarget)));
				}
			}
		}
	}
#else
	{
		// No session services, just add an entry that lets us snapshot ourself
		FWidgetSnapshotTarget SnapshotTarget;
		SnapshotTarget.DisplayName = FText::FromString(FApp::GetInstanceName());
		SnapshotTarget.InstanceId = FApp::GetInstanceId();

		AvailableSnapshotTargets.Add(MakeShareable(new FWidgetSnapshotTarget(SnapshotTarget)));
	}
#endif
}

void SWidgetReflector::UpdateSelectedSnapshotTarget()
{
	if (AvailableSnapshotTargetsComboBox.IsValid())
	{
		const TSharedPtr<FWidgetSnapshotTarget>* FoundSnapshotTarget = AvailableSnapshotTargets.FindByPredicate([this](const TSharedPtr<FWidgetSnapshotTarget>& InAvailableSnapshotTarget) -> bool
		{
			return InAvailableSnapshotTarget->InstanceId == SelectedSnapshotTargetInstanceId;
		});

		if (FoundSnapshotTarget)
		{
			AvailableSnapshotTargetsComboBox->SetSelectedItem(*FoundSnapshotTarget);
		}
		else if (AvailableSnapshotTargets.Num() > 0)
		{
			SelectedSnapshotTargetInstanceId = AvailableSnapshotTargets[0]->InstanceId;
			AvailableSnapshotTargetsComboBox->SetSelectedItem(AvailableSnapshotTargets[0]);
		}
		else
		{
			SelectedSnapshotTargetInstanceId = FGuid();
			AvailableSnapshotTargetsComboBox->SetSelectedItem(nullptr);
		}
	}
}

void SWidgetReflector::OnAvailableSnapshotTargetsChanged()
{
	UpdateAvailableSnapshotTargets();
	UpdateSelectedSnapshotTarget();
}

FText SWidgetReflector::GetSelectedSnapshotTargetDisplayName() const
{
	if (AvailableSnapshotTargetsComboBox.IsValid())
	{
		TSharedPtr<FWidgetSnapshotTarget> SelectedSnapshotTarget = AvailableSnapshotTargetsComboBox->GetSelectedItem();
		if (SelectedSnapshotTarget.IsValid())
		{
			return SelectedSnapshotTarget->DisplayName;
		}
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> SWidgetReflector::HandleGenerateAvailableSnapshotComboItemWidget(TSharedPtr<FWidgetSnapshotTarget> InItem) const
{
	return SNew(STextBlock)
		.Text(InItem->DisplayName);
}

void SWidgetReflector::HandleAvailableSnapshotComboSelectionChanged(TSharedPtr<FWidgetSnapshotTarget> InItem, ESelectInfo::Type InSeletionInfo)
{
	if (InItem.IsValid())
	{
		SelectedSnapshotTargetInstanceId = InItem->InstanceId;
	}
	else
	{
		SelectedSnapshotTargetInstanceId = FGuid();
	}
}

TSharedRef<ITableRow> SWidgetReflector::HandleReflectorTreeGenerateRow( TSharedRef<FWidgetReflectorNodeBase> InReflectorNode, const TSharedRef<STableViewBase>& OwnerTable )
{
	return SNew(SReflectorTreeWidgetItem, OwnerTable)
		.WidgetReflector(SharedThis(this))
		.WidgetInfoToVisualize(InReflectorNode)
		.ToolTip(GenerateToolTipForReflectorNode(InReflectorNode))
		.SourceCodeAccessor(FAccessSourceCode::CreateSP(this, &SWidgetReflector::HandleSourceAccessDelegate))
		.AssetAccessor(AssetAccessDelegate)
		.DebugObjectAccessor(DebugObjectAccessDelegate);
}

void SWidgetReflector::HandleReflectorTreeGetChildren(TSharedRef<FWidgetReflectorNodeBase> InReflectorNode, TArray<TSharedRef<FWidgetReflectorNodeBase>>& OutChildren)
{
	OutChildren = InReflectorNode->GetChildNodes();
}

void SWidgetReflector::HandleReflectorTreeSelectionChanged( TSharedPtr<FWidgetReflectorNodeBase>, ESelectInfo::Type /*SelectInfo*/ )
{
	SelectedNodes = ReflectorTree->GetSelectedItems();

	if (CurrentUIMode == EWidgetReflectorUIMode::Snapshot)
	{
		WidgetSnapshotVisualizer->SetSelectedWidgets(SelectedNodes);
	}

#if WITH_EDITOR
	TArray<UObject*> SelectedWidgetObjects;
	for (TSharedRef<FWidgetReflectorNodeBase>& Node : SelectedNodes)
	{
		TSharedPtr<SWidget> Widget = Node->GetLiveWidget();
		if (Widget.IsValid())
		{
			TSharedPtr<FReflectionMetaData> ReflectinMetaData = Widget->GetMetaData<FReflectionMetaData>();
			if (ReflectinMetaData.IsValid())
			{
				if (UObject* SourceObject = ReflectinMetaData->SourceObject.Get())
				{
					SelectedWidgetObjects.Add(SourceObject);
				}
			}
		}
	}

	if (GIsEditor && SelectedWidgetObjects.Num() > 0)
	{
		if (WidgetReflectorCVars::bEnableFocusOnPick)
		{
			TabManager->TryInvokeTab(WidgetReflectorTabID::WidgetDetails);
		}

		if (PropertyViewPtr.IsValid())
		{
			PropertyViewPtr->SetObjects(SelectedWidgetObjects);
		}
	}
	//else
	//{
	//	CloseTab(WidgetReflectorTabID::WidgetDetails);
	//}
#endif
}

TSharedRef<SWidget> SWidgetReflector::HandleReflectorTreeContextMenu()
{
	// We spawn a large tooltip, close it immediately to prevent context menu from hiding.
	FSlateApplication::Get().CloseToolTip();

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	bool bHasFilteredTreeRoot = ReflectorTreeRoot != FilteredTreeRoot;

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SetAsRootLabel", "Selected node as root"),
		LOCTEXT("SetAsRootTooltip", "Set selected node as the root of the graph"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SWidgetReflector::SetNodesAsReflectorTreeRoot, SelectedNodes),
			FCanExecuteAction::CreateSP(this, &SWidgetReflector::DoesReflectorTreeHasSelectedItem)
		));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowOnlyUMGLabel", "UMG as root"),
		LOCTEXT("ShowOnlyUMGTooltip", "Set UMG as the root of the graph"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SWidgetReflector::HandleStartTreeWithUMG),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SWidgetReflector::HandleIsStartTreeWithUMGEnabled)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);

	MenuBuilder.AddMenuSeparator();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ResetRoot", "Reset filter"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SWidgetReflector::HandleResetFilteredTreeRoot),
			FCanExecuteAction::CreateLambda([bHasFilteredTreeRoot](){ return bHasFilteredTreeRoot; })
		));

	return MenuBuilder.MakeWidget();
}

TSharedPtr<SWidget> SWidgetReflector::HandleReflectorTreeContextMenuPtr()
{
	return HandleReflectorTreeContextMenu();
}

void SWidgetReflector::HandleReflectorTreeHiddenColumnsListChanged()
{
#if WITH_EDITOR
	if (ReflectorTree && ReflectorTree->GetHeaderRow())
	{
		const TArray<FName> HiddenColumnIds = ReflectorTree->GetHeaderRow()->GetHiddenColumnIds();
		HiddenReflectorTreeColumns.Reset(HiddenColumnIds.Num());
		for (const FName Id : HiddenColumnIds)
		{
			HiddenReflectorTreeColumns.Add(Id.ToString());
		}
		SaveSettings();
	}
#endif
}
void SWidgetReflector::CreateCrumbTrailForNode(TSharedRef<FWidgetReflectorNodeBase> InReflectorNode)
{
	FWidgetPath PathToWidget;
	TSharedRef<SWidget> SelectedNodeAsWidget = InReflectorNode.Get().GetLiveWidget().ToSharedRef();
	TArray<TSharedRef<FWidgetReflectorNodeBase>> PickedNodePath;
	FWidgetReflectorNodeUtils::FindLiveWidget(ReflectorTreeRoot, SelectedNodeAsWidget, PickedNodePath);
	BreadCrumb->ClearCrumbs();
	if (PickedNodePath.Num() > 1)
	{
		for (int32 i = 0; i < PickedNodePath.Num(); i++)
		{
			TSharedPtr<SWidget> Parent = PickedNodePath[i]->GetLiveWidget();
			FText WidgetType = FWidgetReflectorNodeUtils::GetWidgetType(Parent);
			BreadCrumb->PushCrumb(WidgetType, PickedNodePath[i]);
		}
	}
}
void SWidgetReflector::HandleReflectorTreeOnMouseClick(TSharedRef<FWidgetReflectorNodeBase> InReflectorNode)
{
	const FModifierKeysState ModKeyState = FSlateApplication::Get().GetModifierKeys();

	if (ModKeyState.IsLeftAltDown())
	{
		if (SelectedNodes.Num() > 0)
		{
			TArray<TSharedRef<FWidgetReflectorNodeBase>> RootNodes;
			RootNodes.Add(InReflectorNode);
			SetNodesAsReflectorTreeRoot(RootNodes);
		}
	}
}

void SWidgetReflector::HandleBreadcrumbOnClick(const TSharedRef<FWidgetReflectorNodeBase>& InReflectorNode)
{
	FilteredTreeRoot.Reset();
	FilteredTreeRoot.Add(InReflectorNode);
	BreadCrumb->PopCrumb();
	ColumnWidths.Empty();
	ReflectorTree->RequestTreeRefresh();
}

TSharedRef< SWidget > SWidgetReflector::HandleBreadcrumbDelimiterMenu(const TSharedRef<FWidgetReflectorNodeBase>& InReflectorNode)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	bool bHasFilteredTreeRoot = ReflectorTreeRoot != FilteredTreeRoot;
	TArray<TSharedRef< FWidgetReflectorNodeBase>> Children = InReflectorNode->GetChildNodes();
	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		TSharedPtr<SWidget> ChildWidget = Children[ChildIndex]->GetLiveWidget();
		FText WidgetType = FWidgetReflectorNodeUtils::GetWidgetType(ChildWidget);
		TArray<TSharedRef<FWidgetReflectorNodeBase>> RootNodes;
		RootNodes.Add(Children[ChildIndex]);
		MenuBuilder.AddMenuEntry(
			WidgetType,
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SWidgetReflector::SetNodesAsReflectorTreeRoot, RootNodes)
			
			));
	}

	return MenuBuilder.MakeWidget();
}

EVisibility SWidgetReflector::HandleIsBreadcrumbVisible() const
{
	return BreadCrumb->HasCrumbs() ? EVisibility::Visible : EVisibility::Collapsed;
}

#if WITH_EDITOR
void SWidgetReflector::HandleEndPIE(const bool bIsSimulating)
{
	if (PropertyViewPtr.IsValid())
	{
		PropertyViewPtr->SetObject(nullptr);
	}

	if (bTransactedPIEObject)
	{
		GEditor->ResetTransaction(LOCTEXT("WidgetReflectorDuringPIE", "PIE Objects may be in the transaction buffer that need to be destroyed"));
		bTransactedPIEObject = false;
	}
}

void SWidgetReflector::HandleObjectTransacted(UObject* InObject, const FTransactionObjectEvent& InTransactionEvent)
{
	// If the transacted object was recorded by the widget reflector.
	if ((GEditor != nullptr && GEditor->PlayWorld != nullptr) || GIsPlayInEditorWorld)
	{
		if (PropertyViewPtr && PropertyViewPtr->GetSelectedObjects().Contains(InObject))
		{
			bTransactedPIEObject = true;
		}
	}
}
#endif //WITH_EDITOR

void SWidgetReflector::HandleResetFilteredTreeRoot()
{
	BreadCrumb->ClearCrumbs();
	bFilterReflectorTreeRootWithUMG = false;
	UpdateFilteredTreeRoot();
	ColumnWidths.Empty();
	ReflectorTree->RequestTreeRefresh();
}

void SWidgetReflector::HandleStartTreeWithUMG()
{
	bFilterReflectorTreeRootWithUMG = !bFilterReflectorTreeRootWithUMG;
	UpdateFilteredTreeRoot();
	ColumnWidths.Empty();
	ReflectorTree->RequestTreeRefresh();
}

// console command

static FDelegateHandle TakeSnapshotEndFrameHandle;

void TakeSnapshotCommand_EndFrame(double RequestedTime, bool bRequestNavigation)
{
	if (RequestedTime <= FApp::GetCurrentTime())
	{
		FWidgetSnapshotData SnapshotData;
		SnapshotData.TakeSnapshot(bRequestNavigation);
		FString Filename = FPaths::CreateTempFilename(*FPaths::GameAgnosticSavedDir(), TEXT(""), TEXT(".widgetsnapshot"));
		SnapshotData.SaveSnapshotToFile(Filename);

		FCoreDelegates::OnEndFrame.Remove(TakeSnapshotEndFrameHandle);
		TakeSnapshotEndFrameHandle.Reset();
	}
}

void TakeSnapshotCommand(const TArray<FString>& Args)
{
	FCoreDelegates::OnEndFrame.Remove(TakeSnapshotEndFrameHandle);

	float RequestedDelay = 0.001f;
	bool bRequestNavigation = false;
	for (const FString& Arg : Args)
	{
		if (FParse::Value(*Arg, TEXT("Delay="), RequestedDelay))
		{
			continue;
		}
		if (FParse::Bool(*Arg, TEXT("Navigation="), bRequestNavigation))
		{
			continue;
		}
	}

	const double CurrentTime = FApp::GetCurrentTime();
	const double RequestedTimeDelay = CurrentTime + (double)RequestedDelay;
	TakeSnapshotEndFrameHandle = FCoreDelegates::OnEndFrame.AddStatic(&TakeSnapshotCommand_EndFrame, RequestedTimeDelay, bRequestNavigation);
}

void DumpFontAtlasesCommand(const TArray<FString>& Args)
{
	FString SaveDirectoryName = FPaths::ProjectSavedDir() / TEXT("WidgetReflector") / TEXT("FontAtlases");
	FString SaveFilePrefix;
	for (const FString& Arg : Args)
	{
		if (FParse::Value(*Arg, TEXT("SaveDirectory"), SaveDirectoryName))
		{
			continue;
		}
		if (FParse::Value(*Arg, TEXT("SaveFilePrefix"), SaveFilePrefix))
		{
			continue;
		}
	}
	
	// We append the timestamp on to the directory names to differentiate between various runs of the command.
	SaveDirectoryName = FString::Format(TEXT("{0}_{1}"), {SaveDirectoryName, FDateTime::Now().ToString()});
	SaveDirectoryName = FPaths::ConvertRelativePathToFull(SaveDirectoryName);
	UE_LOGF(LogSlate, Display, "Dumping all current font atlases to directory '%ls'. All textures will be saved with a user provided file prefix of '%ls'", *SaveDirectoryName, *SaveFilePrefix);
	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
	TSharedRef<FSlateFontCache>  FontCache = FSlateApplication::Get().GetRenderer()->GetFontCache();
	int32 NumFontAtlases = FontCache->GetNumAtlasPages();
	for (int32 Index = 0; Index < NumFontAtlases; ++Index)
	{
		// @TODOFonts: Refactor this out into its own function so the widget reflector could have a button to save a single texture page or save all texture pages 
		ISlateFontTexture* FontTexture = FontCache->GetFontTexture(Index);
		if (!FontTexture)
		{
			UE_LOGF(LogSlate, Warning, "Font texture page %d is not a valid font texture. This page will not be saved.", Index);
			continue;
		}

		TArray<uint8> FontTextureData;
		FontTexture->GetAtlasDataCopy(FontTextureData);
		if (FontTextureData.IsEmpty())
		{
			UE_LOGF(LogSlate, Log, "Texture page %d has no texture data. The texture will not be saved.", Index);
			continue;
		}
		// We need this to get access to the height and width of the texture 
		FSlateShaderResource* ShaderResource = FontTexture->GetSlateTexture();
		if (!ShaderResource)
		{
			UE_LOGF(LogSlate, Log, "Unable to retrieve the shader resource for texture page %d. Skipping saving this texture.", Index);
			continue;
		}
		ESlateFontAtlasContentType ContentType = FontTexture->GetContentType();
		ERawImageFormat::Type ImageFormat = ERawImageFormat::Invalid;
		EGammaSpace GammaSpace = EGammaSpace::Invalid;
		switch (ContentType)
		{
		case ESlateFontAtlasContentType::Alpha:
			ImageFormat = ERawImageFormat::G8;
			GammaSpace = EGammaSpace::Linear;
			break;
		case ESlateFontAtlasContentType::Color:
			ImageFormat = ERawImageFormat::BGRA8;
			GammaSpace = EGammaSpace::sRGB;
			break;
		case ESlateFontAtlasContentType::Msdf:
			ImageFormat = ERawImageFormat::BGRA8;
			GammaSpace = EGammaSpace::Linear;
			break;
		default:
			break;
		}
		FImageView ImageView((void*) FontTextureData.GetData(), ShaderResource->GetWidth(), ShaderResource->GetHeight(), 1 /* InNumSlices*/, ImageFormat, GammaSpace);
		TArray64<uint8> SaveImageBuffer;
		bool bSavedImage = ImageWrapperModule.CompressImage(SaveImageBuffer, EImageFormat::PNG, ImageView);
		if (bSavedImage)
		{
			FString SaveFileName = FString::Format(TEXT("{0}FontAtlasPage_{1}.png"), { SaveFilePrefix, FString::FromInt(Index)});
			FString AbsoluteSaveFileName = SaveDirectoryName / SaveFileName;
			UE_LOGF(LogSlate, Display, "Saving font texture '%ls'", *AbsoluteSaveFileName);
			if (!FFileHelper::SaveArrayToFile(SaveImageBuffer, *AbsoluteSaveFileName))
			{
				UE_LOGF(LogSlate, Warning, "Failed to save font texture '%ls'", *AbsoluteSaveFileName);
			}
		}
		else
		{
			UE_LOGF(LogSlate, Warning, "Failed to convert texture page %d to a buffer to be saved.", Index);
		}
	}
}
} // namespace WidgetReflectorImpl

TSharedRef<SWidgetReflector> SWidgetReflector::New()
{
  return MakeShareable( new WidgetReflectorImpl::SWidgetReflector() );
}

#undef LOCTEXT_NAMESPACE
