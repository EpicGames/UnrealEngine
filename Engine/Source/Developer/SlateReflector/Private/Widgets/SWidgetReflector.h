// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SUserWidget.h"
#include "Framework/Application/IWidgetReflector.h"

class FWidgetReflectorNodeBase;
class FWidgetSnapshotService;

/**
 * Widget reflector implementation.
 * User widget to enable iteration without recompilation.
 */
class SWidgetReflector
	: public SUserWidget
	, public IWidgetReflector
{
public:

	SLATE_USER_ARGS(SWidgetReflector)
	{ }
		
		SLATE_ARGUMENT(TSharedPtr<SDockTab>, ParentTab)
		SLATE_ARGUMENT(TSharedPtr<FWidgetSnapshotService>, WidgetSnapshotService)

	SLATE_END_ARGS()

	virtual void Construct( const FArguments& InArgs ) = 0;
	virtual ~SWidgetReflector() = default;

	virtual void UpdateColumnWidth(TSharedRef<FWidgetReflectorNodeBase> InItem, FName InColumn) = 0;
};

