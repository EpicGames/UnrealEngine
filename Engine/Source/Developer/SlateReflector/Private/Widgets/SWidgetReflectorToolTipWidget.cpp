// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SWidgetReflectorToolTipWidget.h"

#include "Layout/Geometry.h"
#include "Styling/CoreStyle.h"
#include "Types/SlateAttributeMetaData.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/STextBlock.h"

#if WITH_EDITOR
#include "Blueprint/UserWidget.h"
#endif

#define LOCTEXT_NAMESPACE "SReflectorToolTipWidget"

const FText SReflectorToolTipWidget::TrueText = LOCTEXT("True", "True");
const FText SReflectorToolTipWidget::FalseText = LOCTEXT("False", "False");

void SReflectorToolTipWidget::Construct( const FArguments& InArgs )
{
	this->WidgetInfo = InArgs._WidgetInfoToVisualize;
	check(WidgetInfo.IsValid());

	const FGeometry Geom = FGeometry().MakeChild(WidgetInfo->GetLocalSize(), WidgetInfo->GetAccumulatedLayoutTransform(), WidgetInfo->GetAccumulatedRenderTransform(), FVector2D::ZeroVector);
	SizeInfoText = FText::FromString(Geom.ToString());

	bool bIsInsideInvalidationRoot = false;
	FText InvalidationText = FText::GetEmpty();
	{
		TSharedPtr<FWidgetReflectorNodeBase> CurrentWidget = WidgetInfo;
		if (CurrentWidget->GetWidgetIsInvalidationRoot())
		{
			bIsInsideInvalidationRoot = true;
			InvalidationText = LOCTEXT("IsInvalidationRoot", "Is an Invalidation Root");
		}
		else
		{
			while (CurrentWidget)
			{
				if (CurrentWidget->GetWidgetIsInvalidationRoot())
				{
					bIsInsideInvalidationRoot = true;
					InvalidationText = LOCTEXT("FromInvalidationRoot", "Controlled by an Invalidation Root");
					break;
				}
				CurrentWidget = CurrentWidget->GetParentNode();
			}
		}
	}

	TSharedRef<SGridPanel> GridPanel = SNew(SGridPanel)
			.FillColumn(1, 1.0f);
	{
		int32 SlotCount = 0;
		auto BuildLabelAndValue = [GridPanel, &SlotCount](const FText& Label, const TAttribute<FText>& Value)
		{
			GridPanel->AddSlot(0, SlotCount)
			[
				SNew(STextBlock)
				.Text(Label)
			];

			GridPanel->AddSlot(1, SlotCount)
			.Padding(5.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(Value)
			];

			++SlotCount;
		};

		BuildLabelAndValue(LOCTEXT("SourceName", "Source"), { this, &SReflectorToolTipWidget::GetWidgetsSourceName });
		BuildLabelAndValue(LOCTEXT("DesiredSize", "Desired Size"), { this, &SReflectorToolTipWidget::GetWidgetsDesiredSize });
		BuildLabelAndValue(LOCTEXT("ActualSize", "Actual Size"), { this, &SReflectorToolTipWidget::GetWidgetActualSize });
		BuildLabelAndValue(LOCTEXT("SizeInfo", "Size Info"), { this, &SReflectorToolTipWidget::GetSizeInfo });
		BuildLabelAndValue(LOCTEXT("Enabled", "Enabled"), { this, &SReflectorToolTipWidget::GetEnabled });
		BuildLabelAndValue(LOCTEXT("NeedsTick", "Needs Tick"), { this, &SReflectorToolTipWidget::GetNeedsTick });
		BuildLabelAndValue(LOCTEXT("IsVolatile", "Is Volatile"), { this, &SReflectorToolTipWidget::GetIsVolatile });
		BuildLabelAndValue(LOCTEXT("IsVolatileIndirectly", "Is Volatile Indirectly"), { this, &SReflectorToolTipWidget::GetIsVolatileIndirectly });
		if (bIsInsideInvalidationRoot)
		{
			BuildLabelAndValue(LOCTEXT("IsVisible", "Is Visible"), { this, &SReflectorToolTipWidget::GetIsVisible });
			BuildLabelAndValue(LOCTEXT("IsVisibleInherited", "Is Visible Inherited"), { this, &SReflectorToolTipWidget::GetIsVisibleInherited });
			BuildLabelAndValue(LOCTEXT("SupportsInvalidation", "Supports Invalidation"), { this, &SReflectorToolTipWidget::GetSupportsInvalidation });
			BuildLabelAndValue(LOCTEXT("SupportsInvalidationRecursive", "Recursively Supports Invalidation"), { this, &SReflectorToolTipWidget::GetSupportsInvalidationRecursive });
		}
		BuildLabelAndValue(LOCTEXT("HasActiveTimers", "Has Active Timer"), { this, &SReflectorToolTipWidget::GetHasActiveTimers });
		if (WidgetInfo->GetNodeType() == EWidgetReflectorNodeType::Live)
		{
			BuildLabelAndValue(LOCTEXT("HasRegisteredAttribute", "Attributes"), { this, &SReflectorToolTipWidget::GetLiveAttributeName });
		}
		else
		{
			BuildLabelAndValue(LOCTEXT("HasRegisteredAttribute", "Attributes"), { this, &SReflectorToolTipWidget::GetAttributeCount });
			BuildLabelAndValue(LOCTEXT("HasCollapsedAttribute", "Affect Visibility Attributes"), { this, &SReflectorToolTipWidget::GetCollapsedAttributeCount });
		}
	}

	if (bIsInsideInvalidationRoot)
	{
		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(InvalidationText)
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				GridPanel
			]
		];
		
	}
	else
	{
		ChildSlot
		[
			GridPanel
		];
	}
}

FText SReflectorToolTipWidget::GetWidgetsSourceName() const
{
#if WITH_EDITOR
	if (UUserWidget* SourceObject = Cast<UUserWidget>(WidgetInfo->GetWidgetSourceObject()))
	{
		UClass* SourceClass = SourceObject->GetClass();

		return FText::Format(
			LOCTEXT("WidgetTypeInAsset", "{0} in {1}"),
			FText::FromString(SourceClass->ClassGeneratedBy ? SourceClass->ClassGeneratedBy->GetName() : SourceClass->GetName()),
			WidgetInfo->GetWidgetReadableLocation()
		);
	}
#endif

	return WidgetInfo->GetWidgetReadableLocation();
}

FText SReflectorToolTipWidget::GetLiveAttributeName() const
{
	if (TSharedPtr<SWidget> Widget = WidgetInfo->GetLiveWidget())
	{
		if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(*Widget.Get()))
		{
			TArray<FName> AttributeNames = AttributeMetaData->GetAttributeNames(*Widget.Get());
			TArray<FText> AttributeNamesAsText;
			AttributeNamesAsText.Reserve(AttributeNames.Num());
			for (FName AttributeName : AttributeNames)
			{
				AttributeNamesAsText.Add(FText::FromName(AttributeName));
			}
			return FText::FormatOrdered(LOCTEXT("LiveAttributeNames", "{0} ({1} affect widget visibility) \n  {2}")
				, AttributeMetaData->GetRegisteredAttributeCount()
				, AttributeMetaData->GetRegisteredAffectVisibilityAttributeCount()
				, FText::Join(LOCTEXT("LiveAttributeNamesDelimiter", "\n  "), AttributeNamesAsText));
		}
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
