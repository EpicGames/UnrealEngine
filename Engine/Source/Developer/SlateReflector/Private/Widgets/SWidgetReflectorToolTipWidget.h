// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/WidgetReflectorNode.h"

#define LOCTEXT_NAMESPACE "SReflectorToolTipWidget"

class SReflectorToolTipWidget
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SReflectorToolTipWidget)
		: _WidgetInfoToVisualize()
	{ }

		SLATE_ARGUMENT(TSharedPtr<FWidgetReflectorNodeBase>, WidgetInfoToVisualize)

	SLATE_END_ARGS()

public:

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

private:

	FText GetWidgetsSourceName() const;

	FText GetWidgetsDesiredSize() const
	{
		return FText::FromString(WidgetInfo->GetWidgetDesiredSize().ToString());
	}

	FText GetWidgetActualSize() const
	{
		return FText::FromString(WidgetInfo->GetLocalSize().ToString());
	}

	FText GetSizeInfo() const
	{
		return SizeInfoText;
	}

	FText GetEnabled() const
	{
		return (WidgetInfo->GetWidgetEnabled()) ? TrueText : FalseText;
	}

	FText GetNeedsTick() const
	{
		return (WidgetInfo->GetWidgetNeedsTick()) ? TrueText : FalseText;
	}

	FText GetIsVolatile() const
	{
		return (WidgetInfo->GetWidgetIsVolatile()) ? TrueText : FalseText;
	}

	FText GetIsVolatileIndirectly() const
	{
		return (WidgetInfo->GetWidgetIsVolatileIndirectly()) ? TrueText : FalseText;
	}

	FText GetHasActiveTimers() const
	{
		return (WidgetInfo->GetWidgetHasActiveTimers()) ? TrueText : FalseText;
	}

	FText GetLiveAttributeName() const;

	FText GetAttributeCount() const
	{
		return FText::AsNumber(WidgetInfo->GetWidgetAttributeCount());
	}

	FText GetCollapsedAttributeCount() const
	{
		return FText::AsNumber(WidgetInfo->GetWidgetCollapsedAttributeCount());
	}

	FText GetIsVisible() const
	{
		return (WidgetInfo->GetWidgetVisible()) ? TrueText : FalseText;
	}
	
	FText GetIsVisibleInherited() const
	{
		return (WidgetInfo->GetWidgetVisibilityInherited()) ? TrueText : FalseText;
	}

	FText GetSupportsInvalidation() const
	{
		return (WidgetInfo->GetWidgetSupportsInvalidation()) ? TrueText : FalseText;
	}
	
	FText GetSupportsInvalidationRecursive() const
	{
		return (WidgetInfo->GetSupportsInvalidationRecursive()) ? TrueText : FalseText;
	}

private:

	static const FText TrueText;
	static const FText FalseText;

	/** The info about the widget that we are visualizing. */
	TSharedPtr<FWidgetReflectorNodeBase> WidgetInfo;

	/** The size info text */
	FText SizeInfoText;
};


#undef LOCTEXT_NAMESPACE
