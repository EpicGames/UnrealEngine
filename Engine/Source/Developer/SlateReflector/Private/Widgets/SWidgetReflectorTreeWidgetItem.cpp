// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SWidgetReflectorTreeWidgetItem.h"
#include "SlateOptMacros.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SHyperlink.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/SInvalidationPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SWidgetReflector.h"

#if WITH_EDITOR
#include "Blueprint/UserWidget.h"
#endif

#define LOCTEXT_NAMESPACE "SWidgetReflector"

/* SMultiColumnTableRow overrides
 *****************************************************************************/

FName SReflectorTreeWidgetItem::NAME_WidgetName(TEXT("WidgetName"));
FName SReflectorTreeWidgetItem::NAME_WidgetInfo(TEXT("WidgetInfo"));
FName SReflectorTreeWidgetItem::NAME_Visibility(TEXT("Visibility"));
FName SReflectorTreeWidgetItem::NAME_Focusable(TEXT("Focusable"));
FName SReflectorTreeWidgetItem::NAME_Enabled(TEXT("Enabled"));
FName SReflectorTreeWidgetItem::NAME_Volatile(TEXT("Volatile"));
FName SReflectorTreeWidgetItem::NAME_HasActiveTimer(TEXT("HasActiveTimer"));
FName SReflectorTreeWidgetItem::NAME_Clipping(TEXT("Clipping"));
FName SReflectorTreeWidgetItem::NAME_LayerId(TEXT("LayerId"));
FName SReflectorTreeWidgetItem::NAME_ForegroundColor(TEXT("ForegroundColor"));
FName SReflectorTreeWidgetItem::NAME_ActualSize(TEXT("ActualSize"));
FName SReflectorTreeWidgetItem::NAME_Address(TEXT("Address"));

void SReflectorTreeWidgetItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	this->WidgetReflectorWeak = InArgs._WidgetReflector;
	this->WidgetInfo = InArgs._WidgetInfoToVisualize;
	this->OnAccessSourceCode = InArgs._SourceCodeAccessor;
	this->OnAccessAsset = InArgs._AssetAccessor;
	this->OnAccessDebugObject = InArgs._DebugObjectAccessor;
	this->SetPadding(0.f);

	check(WidgetInfo.IsValid());

	SMultiColumnTableRow< TSharedRef<FWidgetReflectorNodeBase> >::Construct(SMultiColumnTableRow< TSharedRef<FWidgetReflectorNodeBase> >::FArguments().Padding(0.f), InOwnerTableView);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

// Instead of changing the big method below, wrap it with this one to update the maximum width needed for the column.
TSharedRef<SWidget> SReflectorTreeWidgetItem::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedRef<SWidget> Widget = GenerateWidgetForColumn_Internal(ColumnName);

	if (ColumnName == NAME_WidgetName
		|| ColumnName == NAME_WidgetInfo
		|| ColumnName == NAME_LayerId
		|| ColumnName == NAME_ActualSize)
	{
		ColumnsToUpdate.Add(ColumnName);
	}

	return Widget;
}

TSharedRef<SWidget> SReflectorTreeWidgetItem::GenerateWidgetForColumn_Internal(const FName& ColumnName)
{
	SReflectorTreeWidgetItem* Self = this;
	auto BuildCheckBox = [Self](bool bIsChecked)
		{
			return SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.0f, 0.0f))
				[
					SNew(SCheckBox)
					.Style(FCoreStyle::Get(), TEXT("WidgetReflector.FocusableCheck"))
					.IsChecked(bIsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
				];
		};

	if (ColumnName == NAME_WidgetName )
	{
		TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

		HorizontalBox->AddSlot()
		.AutoWidth()
		[
			SNew(SExpanderArrow, SharedThis(this))
			.IndentAmount(16)
			.ShouldDrawWires(true)
		];

#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
		// Here we show an indication for Dynamic Invalidation Panels and their children.
		// Dynamic Invalidation Panels are Invalidation Panels with UseDynamicInvalidation set to true.
		static FName InvalidationPanelName = TEXT("SInvalidationPanel");

		const TSharedPtr<SWidget> LiveWidget = WidgetInfo->GetLiveWidget();
		if (LiveWidget && LiveWidget != SNullWidget::NullWidget)
		{
			TSharedPtr<SWidget> InvalidationPanel = LiveWidget;
			while (InvalidationPanel && InvalidationPanel->GetType() != InvalidationPanelName)
			{
				InvalidationPanel = InvalidationPanel->GetParentWidget();
			}

			TOptional<TPair<FLinearColor, FText>> IndicatorColorAndTooltip;
			if (WidgetInfo->GetParentNode() && LiveWidget->GetParentWidget() != WidgetInfo->GetParentNode()->GetLiveWidget())
			{
				IndicatorColorAndTooltip = TPair<FLinearColor, FText> {
					FLinearColor(0.5f, 0.0f, 0.f), // Dark Red for widget with wrong parent
					LOCTEXT("DynamicInvalidationPanel_WidgetWithWrongParent", "Widget's 'GetParentWidget' is not its current parent.")
				};
			}
			else if (InvalidationPanel && StaticCastSharedPtr<SInvalidationPanel>(InvalidationPanel)->CanUseDynamicInvalidation()) // if we found an Invalidation Panel using Dynamic Invalidation
			{
				const bool CanCache = StaticCastSharedPtr<SInvalidationPanel>(InvalidationPanel)->GetCanCache();
				// we always show a color block for the Dynamic Invalidation Panel, but for its children we only show the color if it can cache
				const bool bShowCachingIndicator = CanCache || LiveWidget == InvalidationPanel;
				if (bShowCachingIndicator)
				{
					if (LiveWidget == InvalidationPanel)
					{
						if (!CanCache)
						{
							IndicatorColorAndTooltip = TPair<FLinearColor, FText> {
								FLinearColor(1.0f, 0.0f, 0.0f), // Red for disabled Dynamic Invalidation Panel
								LOCTEXT("DynamicInvalidationPanel_Disabled", "Dynamic Invalidation Panel cannot cache")
							};
						}
						else if (LiveWidget->SupportsInvalidationRecursive(EInvalidationStrategy::UseCachedValue))
						{
							IndicatorColorAndTooltip = TPair<FLinearColor, FText> {
								FLinearColor(0.0f, 0.5f, 1.0f), // Blue for Dynamic Invalidation Panel that recursively supports invalidation
								LOCTEXT("DynamicInvalidationPanel_SupportingInvalidationRecursive", "Dynamic Invalidation Panel is caching all its children")
							} ;
						}
						else
						{
							IndicatorColorAndTooltip = TPair<FLinearColor, FText> {
								FLinearColor(1.f, 0.f, 1.f), // Magenta for Dynamic Invalidation Panel that is not able to recursively supports invalidation
								LOCTEXT("DynamicInvalidationPanel_NotSupportingInvalidationRecursive", "Dynamic Invalidation Panel supports invalidation but not all its children do")
							};
						}
					}
					else
					{
						if (!LiveWidget->SupportsInvalidation())
						{
							IndicatorColorAndTooltip = TPair<FLinearColor, FText> {
								FLinearColor(1.0f, 0.0f, 0.0f), // Red for widgets not supporting invalidation
								LOCTEXT("DynamicInvalidationPanel_WidgetNotSupportingInvalidation", "Widget Class does not support invalidation")
							};
						}
						else if (LiveWidget->SupportsInvalidationRecursive(EInvalidationStrategy::UseCachedValue))
						{
							IndicatorColorAndTooltip = TPair<FLinearColor, FText> {
								FLinearColor(0.0f, 1.0f, 0.0f), // Green for widgets recursively supporting invalidation
								LOCTEXT("DynamicInvalidationPanel_WidgetSupportingInvalidationRecursive", "Widget and children all support invalidation")
							};
						}
						else
						{
							IndicatorColorAndTooltip = TPair<FLinearColor, FText> {
								FLinearColor(1.0f, .5f, 0.0f), // Orange for widgets supporting invalidation but having children that do not
								LOCTEXT("DynamicInvalidationPanel_WidgetNotSupportingInvalidationRecursive", "Widget supports invalidation but not all its children do")
							};
						}
					}
				}
			}

			if (IndicatorColorAndTooltip.IsSet())
			{
				HorizontalBox->AddSlot()
					.AutoWidth()
					.Padding(4.0f, 0.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SColorBlock)
						.Size(FVector2D{10.0f, 10.0f})
						.Color(IndicatorColorAndTooltip.GetValue().Key)
						.ToolTipText(FText::Format(LOCTEXT("DynamicInvalidationPanel_TooltipFormat", "{0}\nCached Result: {1}"),
							IndicatorColorAndTooltip.GetValue().Value,
							FText::FromString(LexToString(LiveWidget->Debug_GetCachedInvalidationResult()))
						))
					];
			}
		}
#endif

		if (WidgetInfo->GetWidgetIsInvalidationRoot())
		{
			HorizontalBox->AddSlot()
			.AutoWidth()
			.Padding(4.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("InvalidationRoot_Short", "[IR]"))
			];
		}

		HorizontalBox->AddSlot()
		.AutoWidth()
		.Padding(4.0f, 0.0f, 9.f, 0.f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(WidgetInfo->GetWidgetTypeAndShortName())
			.ColorAndOpacity(this, &SReflectorTreeWidgetItem::GetTint)
		];

		return HorizontalBox;
	}
	else if (ColumnName == NAME_WidgetInfo )
	{
		TSharedRef<SHorizontalBox> Info = SNew(SHorizontalBox);

#if WITH_EDITOR
		if (UUserWidget* SourceObject = Cast<UUserWidget>(WidgetInfo->GetWidgetSourceObject()))
		{
			UClass* SourceClass = SourceObject->GetClass();

			Info->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.f, 0.f, 2.f, 0.f))
				[
					SNew(SHyperlink)
					.Text(SourceClass->GetDisplayNameText())
					.OnNavigate(this, &SReflectorTreeWidgetItem::HandleHyperlinkNavigate, /* Attempt class debug */ true)
				];

			Info->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Source_In", "in"))
				];
			
			Info->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(FMargin(2.f, 0.f, 4.f, 0.f))
				[
					SNew(SHyperlink)
					.Text(WidgetInfo->GetWidgetReadableLocation())
					.OnNavigate(this, &SReflectorTreeWidgetItem::HandleHyperlinkNavigate, /* Attempt class debug */ false)
				];
		}
		else
		{
#endif
			Info->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(FMargin(2.f, 0.f))
				[
					SNew(SHyperlink)
					.Text(WidgetInfo->GetWidgetReadableLocation())
					.OnNavigate(this, &SReflectorTreeWidgetItem::HandleHyperlinkNavigate, /* Attempt class debug */ false)
				];
#if WITH_EDITOR
		}
#endif
		return Info;
	}
	else if (ColumnName == NAME_Visibility )
	{
		return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(WidgetInfo->GetWidgetVisibilityText())
					.Justification(ETextJustify::Center)
			];
	}
	else if (ColumnName == NAME_Focusable)
	{
		return BuildCheckBox(WidgetInfo->GetWidgetFocusable());
	}
	else if (ColumnName == NAME_Enabled)
	{
		return BuildCheckBox(WidgetInfo->GetWidgetEnabled());
	}
	else if (ColumnName == NAME_Volatile)
	{
		return BuildCheckBox(WidgetInfo->GetWidgetIsVolatile());
	}
	else if (ColumnName == NAME_HasActiveTimer)
	{
		return BuildCheckBox(WidgetInfo->GetWidgetHasActiveTimers());
	}
	else if ( ColumnName == NAME_Clipping )
	{
		return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(WidgetInfo->GetWidgetClippingText())
			];
	}
	else if (ColumnName == NAME_LayerId)
	{
		return SNew(SBox)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("WidgetLayerIds", "[{0}, {1}]"), FText::AsNumber(WidgetInfo->GetWidgetLayerId()), FText::AsNumber(WidgetInfo->GetWidgetLayerIdOut())))
			];
	}
	else if (ColumnName == NAME_ForegroundColor )
	{
		const FSlateColor Foreground = WidgetInfo->GetWidgetForegroundColor();

		return SNew(SBorder)
			// Show unset color as an empty space.
			.Visibility(Foreground.IsColorSpecified() ? EVisibility::Visible : EVisibility::Hidden)
			// Show a checkerboard background so we can see alpha values well
			.BorderImage(FCoreStyle::Get().GetBrush("Checkerboard"))
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.0f, 0.0f))
			[
				// Show a color block
				SNew(SColorBlock)
				.Color(Foreground.GetSpecifiedColor())
				.Size(FVector2D(16.0f, 16.0f))
			];
	}
	else if (ColumnName == NAME_ActualSize)
	{
		return SNew(SBox)
			.Padding(FMargin(4.f, 0.f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(WidgetInfo->GetLocalSize().ToString()))
			];
	}
	else if (ColumnName == NAME_Address )
	{
		const FString WidgetAddress = FWidgetReflectorNodeUtils::WidgetAddressToString(WidgetInfo->GetWidgetAddress());
		const FText Address = FText::FromString(WidgetAddress);
		const FString ConditionalBreakPoint = FString::Printf(TEXT("this == (SWidget*)%s"), *WidgetAddress);

		return SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(SHyperlink)
				.ToolTipText(LOCTEXT("ClickToCopyBreakpoint", "Click to copy conditional breakpoint for this instance."))
				.Text(LOCTEXT("CBP", "[CBP]"))
				.OnNavigate_Lambda([ConditionalBreakPoint](){ FPlatformApplicationMisc::ClipboardCopy(*ConditionalBreakPoint); })
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
			[
				SNew(SHyperlink)
				.ToolTipText(LOCTEXT("ClickToCopy", "Click to copy address."))
				.Text(Address)
				.OnNavigate_Lambda([Address]() { FPlatformApplicationMisc::ClipboardCopy(*Address.ToString()); })
			];
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

int32 SReflectorTreeWidgetItem::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 NewLayerId = SMultiColumnTableRow<TSharedRef<FWidgetReflectorNodeBase>>::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	// Deferred update now we have a depth.
	if (!ColumnsToUpdate.IsEmpty() && WidgetInfo.IsValid())
	{
		if (TSharedPtr<SWidgetReflector> WidgetReflector = WidgetReflectorWeak.Pin())
		{
			for (const FName& ColumnName : ColumnsToUpdate)
			{
				WidgetReflector->UpdateColumnWidth(WidgetInfo.ToSharedRef(), ColumnName);
			}
		}

		ColumnsToUpdate.Empty();
	}

	return NewLayerId;
}

void SReflectorTreeWidgetItem::HandleHyperlinkNavigate(bool bAttemptClassDebug)
{
#if WITH_EDITOR
	bool bHasOpenedAsset = false;

	if (bAttemptClassDebug)
	{
		if (UUserWidget* SourceObject = Cast<UUserWidget>(WidgetInfo->GetWidgetSourceObject()))
		{
			UClass* SourceClass = SourceObject->GetClass();

			if (SourceClass->ClassGeneratedBy)
			{
				OnAccessAsset.Execute(SourceClass->ClassGeneratedBy);
				bHasOpenedAsset = true;

				if (OnAccessDebugObject.IsBound())
				{
					OnAccessDebugObject.Execute(SourceObject);
				}
			}
		}
	}

	if (!bHasOpenedAsset)
	{
#endif
		FAssetData CachedAssetData = WidgetInfo->GetWidgetAssetData();

		if (CachedAssetData.IsValid())
		{
			if (OnAccessAsset.IsBound())
			{
				CachedAssetData.GetPackage();
				OnAccessAsset.Execute(CachedAssetData.GetAsset());
			}
		}
#if WITH_EDITOR
	}
#endif

	if (OnAccessSourceCode.IsBound())
	{
		const FString WidgetFile = WidgetInfo->GetWidgetFile();
		const FString WidgetFilePath = TEXT("\\Engine\\Source\\Runtime\\UMG\\Private\\Components\\Widget.cpp");
		const FString UserWidgetFilePath = TEXT("\\Engine\\Source\\Runtime\\UMG\\Private\\Components\\UserWidget.cpp");

		if (!WidgetFile.EndsWith(WidgetFilePath) && !WidgetFile.EndsWith(UserWidgetFilePath))
		{
			OnAccessSourceCode.Execute(WidgetInfo->GetWidgetFile(), WidgetInfo->GetWidgetLineNumber(), 0);
		}
	}
}

#undef LOCTEXT_NAMESPACE