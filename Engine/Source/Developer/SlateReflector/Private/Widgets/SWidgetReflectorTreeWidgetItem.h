// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Models/WidgetReflectorNode.h"

class SWidgetReflector;

/**
 * Widget that visualizes the contents of a FReflectorNode.
 */
class SReflectorTreeWidgetItem
	: public SMultiColumnTableRow<TSharedRef<FWidgetReflectorNodeBase>>
{
public:

	static FName NAME_WidgetName;
	static FName NAME_WidgetInfo;
	static FName NAME_Visibility;
	static FName NAME_Focusable;
	static FName NAME_Enabled;
	static FName NAME_Volatile;
	static FName NAME_HasActiveTimer;
	static FName NAME_Clipping;
	static FName NAME_LayerId;
	static FName NAME_ForegroundColor;
	static FName NAME_Address;
	static FName NAME_ActualSize;

	SLATE_BEGIN_ARGS(SReflectorTreeWidgetItem)
		: _WidgetReflector()
		, _WidgetInfoToVisualize()
		, _SourceCodeAccessor()
		, _AssetAccessor()
	{ }
		SLATE_ARGUMENT(TSharedPtr<SWidgetReflector>, WidgetReflector)
		SLATE_ARGUMENT(TSharedPtr<FWidgetReflectorNodeBase>, WidgetInfoToVisualize)
		SLATE_ARGUMENT(FAccessSourceCode, SourceCodeAccessor)
		SLATE_ARGUMENT(FAccessAsset, AssetAccessor)
		SLATE_ARGUMENT(FAccessDebugObject, DebugObjectAccessor)

	SLATE_END_ARGS()

public:

	/**
	 * Construct child widgets that comprise this widget.
	 *
	 * @param InArgs Declaration from which to construct this widget.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

public:

	// SMultiColumnTableRow overrides
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override;

	// Expose parent class protected function.
	using SMultiColumnTableRow<TSharedRef<FWidgetReflectorNodeBase>>::GetWidgetFromColumnId;

protected:

	/** @return The tint of the reflector node */
	FSlateColor GetTint() const
	{
		return WidgetInfo->GetTint();
	}

	void HandleHyperlinkNavigate(bool bAttemptClassDebug);

	TSharedRef<SWidget> GenerateWidgetForColumn_Internal(const FName& ColumnName);

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

private:
	TWeakPtr<SWidgetReflector> WidgetReflectorWeak;

	/** The info about the widget that we are visualizing. */
	TSharedPtr<FWidgetReflectorNodeBase> WidgetInfo;

	FAccessSourceCode OnAccessSourceCode;
	FAccessAsset OnAccessAsset;
	FAccessDebugObject OnAccessDebugObject;

	mutable TSet<FName> ColumnsToUpdate;
};
