// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SWidgetSearchPanel.h"

#include "Framework/Application/SlateApplication.h"
#include "Models/WidgetReflectorNode.h"
#include "Styling/CoreStyle.h"
#include "Styling/StyleColors.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "SWidgetSearchPanel"

namespace UE::SlateReflector
{

namespace WidgetSearchColumnID
{
	static const FName DisplayName = "DisplayName";
	static const FName Type = "Type";
	static const FName Tag = "Tag";
	static const FName Source = "Source";
	static const FName Address = "Address";
}

// ─── Result entry ────────────────────────────────────────────────────────────

	/** A single entry displayed in the search results list */
struct FWidgetSearchResultEntry
{
	TSharedRef<FWidgetReflectorNodeBase> Node;
	FText DisplayName;
	FText Type;
	FText Tag;
	FText Source;
	FText Address;

	explicit FWidgetSearchResultEntry(TSharedRef<FWidgetReflectorNodeBase> InNode)
		: Node(InNode)
	{
		DisplayName = Node->GetWidgetTypeAndShortName();
		Type = Node->GetWidgetType();
		Source = Node->GetWidgetReadableLocation();
		Address = FText::FromString(FWidgetReflectorNodeUtils::WidgetAddressToString(Node->GetWidgetAddress()));

		// Tag is only available for live widgets; snapshot nodes return nullptr from GetLiveWidget().
		if (TSharedPtr<SWidget> LiveWidget = Node->GetLiveWidget())
		{
			const FName TagName = LiveWidget->GetTag();
			Tag = TagName.IsNone() ? FText::GetEmpty() : FText::FromName(TagName);
		}
	}
};

// ─── Row widget ──────────────────────────────────────────────────────────────

class SWidgetSearchResultRow : public SMultiColumnTableRow<TSharedPtr<FWidgetSearchResultEntry>>
{
public:
	SLATE_BEGIN_ARGS(SWidgetSearchResultRow) {}
		SLATE_ARGUMENT(TSharedPtr<FWidgetSearchResultEntry>, Entry)
		SLATE_ARGUMENT(FAccessSourceCode, SourceCodeAccessor)
		SLATE_ARGUMENT(FAccessAsset, AssetAccessor)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		Entry = InArgs._Entry;
		OnAccessSourceCode = InArgs._SourceCodeAccessor;
		OnAccessAsset = InArgs._AssetAccessor;
		FSuperRowType::Construct(FSuperRowType::FArguments(), InOwnerTable);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (!Entry.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		if (ColumnName == WidgetSearchColumnID::Source)
		{
			return SNew(SBox)
				.Padding(FMargin(4.f, 2.f))
				.VAlign(VAlign_Center)
				[
					SNew(SHyperlink)
					.Text(Entry->Source)
					.ToolTipText(Entry->Source)
					.OnNavigate(this, &SWidgetSearchResultRow::HandleSourceNavigation)
				];
		}

		if (ColumnName == WidgetSearchColumnID::Address)
		{
			return SNew(SBox)
				.Padding(FMargin(4.f, 2.f))
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SHyperlink)
						.ToolTipText(LOCTEXT("ClickToCopyBreakpoint", "Click to copy conditional breakpoint for this instance."))
						.Text(LOCTEXT("CBP", "[CBP]"))
						.OnNavigate(this, &SWidgetSearchResultRow::HandleConditionalBreakpoint)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
					[
						SNew(SHyperlink)
						.ToolTipText(LOCTEXT("ClickToCopyAddress", "Click to copy address."))
						.Text(Entry->Address)
						.OnNavigate(this, &SWidgetSearchResultRow::HandleClickToCopyAddress)
					]
				];
		}

		FText ColumnText;
		if (ColumnName == WidgetSearchColumnID::DisplayName)
		{
			ColumnText = Entry->DisplayName;
		}
		else if (ColumnName == WidgetSearchColumnID::Type)
		{
			ColumnText = Entry->Type;
		}
		else if (ColumnName == WidgetSearchColumnID::Tag)
		{
			ColumnText = Entry->Tag;
		}

		return SNew(SBox)
			.Padding(FMargin(4.f, 2.f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(ColumnText)
				.ToolTipText(ColumnText)
			];
	}

private:
	void HandleSourceNavigation()
	{
		const FAssetData& AssetData = Entry->Node->GetWidgetAssetData();
		if (AssetData.IsValid() && OnAccessAsset.IsBound())
		{
			OnAccessAsset.Execute(AssetData.GetAsset());
		}
		else if (OnAccessSourceCode.IsBound())
		{
			OnAccessSourceCode.Execute(Entry->Node->GetWidgetFile(), Entry->Node->GetWidgetLineNumber(), 0);
		}
	}

	void HandleConditionalBreakpoint()
	{
		const FString ConditionalBreakpoint = FString::Printf(TEXT("this == (SWidget*)%s"), *Entry->Address.ToString());
		FPlatformApplicationMisc::ClipboardCopy(*ConditionalBreakpoint);
	}

	void HandleClickToCopyAddress()
	{
		FPlatformApplicationMisc::ClipboardCopy(*Entry->Address.ToString());
	}

private:
	TSharedPtr<FWidgetSearchResultEntry> Entry;
	FAccessSourceCode OnAccessSourceCode;
	FAccessAsset OnAccessAsset;
};

// ─── Panel ───────────────────────────────────────────────────────────────────

void SWidgetSearchPanel::Construct(const FArguments& InArgs)
{
	OnSelectNode = InArgs._OnSelectNode;
	OnSelectLiveWidget = InArgs._OnSelectLiveWidget;
	OnGetRootNodes = InArgs._OnGetRootNodes;
	OnAccessSourceCode = InArgs._OnAccessSourceCode;
	OnAccessAsset = InArgs._OnAccessAsset;

	// Helper: create a fixed-width right-aligned label cell
	auto MakeLabel = [](FText InText) -> TSharedRef<SWidget>
	{
		return SNew(SBox)
			.WidthOverride(90.f)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 6.f, 0.f))
			[
				SNew(STextBlock).Text(InText)
			];
	};

	// Helper: build one label+textbox row
	auto MakeRow = [&](FText InLabel, TSharedPtr<SEditableTextBox>& OutBox) -> TSharedRef<SWidget>
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				MakeLabel(InLabel)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SAssignNew(OutBox, SEditableTextBox)
				.ClearKeyboardFocusOnCommit(false)
			];
	};

	// Build the search form as a vertical stack of rows
	TSharedRef<SVerticalBox> SearchForm = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 0.f, 0.f, 4.f)
		[
			MakeRow(LOCTEXT("DescLabel", "Widget Name:"), DescSearchBox)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 0.f, 0.f, 4.f)
		[
			MakeRow(LOCTEXT("TypeLabel", "Type:"), TypeSearchBox)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 0.f, 0.f, 4.f)
		[
			MakeRow(LOCTEXT("TagLabel", "Tag:"), TagSearchBox)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 0.f, 0.f, 4.f)
		[
			MakeRow(LOCTEXT("AddressLabel", "Address:"), AddressSearchBox)
		]

		// ── Scope selector + Search button on one row ─────────────────────
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 6.f, 0.f, 0.f)
		[
			SNew(SHorizontalBox)

			// Checkbox: search all live widgets vs. current tree root
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SAssignNew(AllLiveWidgetsCheckBox, SCheckBox)
				.ToolTipText(LOCTEXT("AllLiveWidgetsTip",
					"When checked, the search builds a fresh widget tree from every visible window "
					"instead of searching only the nodes currently shown in the Widget Hierarchy.\n"
					"Note: selecting a result from an all-live search will focus it in the hierarchy "
					"only if that window has already been captured there."))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AllLiveWidgetsLabel", "Search all live widgets"))
				]
			]

			// Spacer pushes the Search button to the right
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.Text(LOCTEXT("SearchButton", "Search"))
				.OnClicked(this, &SWidgetSearchPanel::HandleSearchClicked)
			]
		];

	// Build the results list
	TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow)
		+ SHeaderRow::Column(WidgetSearchColumnID::DisplayName)
		.DefaultLabel(LOCTEXT("ColWidgetName", "Widget Name"))
		.FillWidth(2.f)
		+ SHeaderRow::Column(WidgetSearchColumnID::Type)
		.DefaultLabel(LOCTEXT("ColType", "Type"))
		.FillWidth(1.5f)
		+ SHeaderRow::Column(WidgetSearchColumnID::Tag)
		.DefaultLabel(LOCTEXT("ColTag", "Tag"))
		.FillWidth(1.f)
		+ SHeaderRow::Column(WidgetSearchColumnID::Source)
		.DefaultLabel(LOCTEXT("ColSource", "Source"))
		.FillWidth(2.f)
		+ SHeaderRow::Column(WidgetSearchColumnID::Address)
		.DefaultLabel(LOCTEXT("ColAddress", "Address"))
		.FillWidth(1.5f);

	SAssignNew(ResultListView, SListView<TSharedPtr<FWidgetSearchResultEntry>>)
		.ListItemsSource(&SearchResults)
		.OnGenerateRow(this, &SWidgetSearchPanel::HandleGenerateRow)
		.OnSelectionChanged(this, &SWidgetSearchPanel::HandleSelectionChanged)
		.HeaderRow(HeaderRow)
		.SelectionMode(ESelectionMode::Single);

	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Vertical)

		+ SSplitter::Slot()
		.Value(0.35f)
		[
			SNew(SBorder)
			.Padding(FMargin(8.f))
			.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			[
				SearchForm
			]
		]

		+ SSplitter::Slot()
		.Value(0.65f)
		[
			SNew(SBorder)
			.Padding(FMargin(0.f))
			.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			[
				ResultListView.ToSharedRef()
			]
		]
	];
}

bool SWidgetSearchPanel::IsAllLiveWidgetsMode() const
{
	return AllLiveWidgetsCheckBox.IsValid() && AllLiveWidgetsCheckBox->IsChecked();
}

TArray<TSharedRef<FWidgetReflectorNodeBase>> SWidgetSearchPanel::BuildSearchRoots() const
{
	if (IsAllLiveWidgetsMode())
	{
		// Build a transient live tree covering every visible window.
		TArray<TSharedRef<SWindow>> VisibleWindows;
		FSlateApplication::Get().GetAllVisibleWindowsOrdered(VisibleWindows);

		TArray<TSharedRef<FWidgetReflectorNodeBase>> Roots;
		Roots.Reserve(VisibleWindows.Num());
		for (const TSharedRef<SWindow>& Window : VisibleWindows)
		{
			Roots.Add(FWidgetReflectorNodeUtils::NewLiveNodeTreeFrom(FArrangedWidget(Window, Window->GetWindowGeometryInScreen())));
		}
		return Roots;
	}

	// Fall back to the current tree root provided by the reflector.
	if (OnGetRootNodes.IsBound())
	{
		return OnGetRootNodes.Execute();
	}
	return {};
}

FReply SWidgetSearchPanel::HandleSearchClicked()
{
	FSearchFilters Filters;
	Filters.Type = TypeSearchBox->GetText().ToString();
	Filters.DisplayName = DescSearchBox->GetText().ToString();
	Filters.Tag = TagSearchBox->GetText().ToString();
	Filters.Address = AddressSearchBox->GetText().ToString();

	// Build and cache the search roots so that result entries remain valid after the search.
	LastSearchRoots = BuildSearchRoots();

	SearchResults.Reset();
	SearchNodes(LastSearchRoots, Filters);

	if (ResultListView.IsValid())
	{
		ResultListView->RequestListRefresh();
	}

	return FReply::Handled();
}

void SWidgetSearchPanel::SearchNodes(const TArray<TSharedRef<FWidgetReflectorNodeBase>>& InNodes, const FSearchFilters& Filters)
{
	TArray<TSharedRef<FWidgetReflectorNodeBase>> Nodes = InNodes;
	while (!Nodes.IsEmpty())
	{
		const TSharedRef<FWidgetReflectorNodeBase> Node = Nodes.Pop();
		if (NodeMatchesFilters(Node, Filters))
		{
			SearchResults.Add(MakeShared<FWidgetSearchResultEntry>(Node));
		}

		// Always recurse into children regardless of match
		for (const TSharedRef<FWidgetReflectorNodeBase>& Child : Node->GetChildNodes())
		{
			Nodes.Add(Child);
		}
	}
}

bool SWidgetSearchPanel::NodeMatchesFilters(const TSharedRef<FWidgetReflectorNodeBase>& Node, const FSearchFilters& Filters) const
{
	// If all filters are empty, nothing matches (require at least one criteria)
	if (Filters.IsEmpty())
	{
		return false;
	}

	// Each non-empty filter must match (AND logic)
	if (!Filters.Type.IsEmpty())
	{
		const FString NodeType = Node->GetWidgetType().ToString();
		if (!NodeType.Contains(Filters.Type, ESearchCase::IgnoreCase))
		{
			return false;
		}
	}

	if (!Filters.DisplayName.IsEmpty())
	{
		const FString NodeDesc = Node->GetWidgetTypeAndShortName().ToString();
		if (!NodeDesc.Contains(Filters.DisplayName, ESearchCase::IgnoreCase))
		{
			return false;
		}
	}

	if (!Filters.Tag.IsEmpty())
	{
		FString NodeTag;
		if (TSharedPtr<SWidget> LiveWidget = Node->GetLiveWidget())
		{
			NodeTag = LiveWidget->GetTag().ToString();
		}
		if (!NodeTag.Contains(Filters.Tag, ESearchCase::IgnoreCase))
		{
			return false;
		}
	}

	if (!Filters.Address.IsEmpty())
	{
		const FString NodeAddress = FWidgetReflectorNodeUtils::WidgetAddressToString(Node->GetWidgetAddress());
		if (!NodeAddress.Contains(Filters.Address, ESearchCase::IgnoreCase))
		{
			return false;
		}
	}

	return true;
}

TSharedRef<ITableRow> SWidgetSearchPanel::HandleGenerateRow(TSharedPtr<FWidgetSearchResultEntry> Item,
	const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SWidgetSearchResultRow, OwnerTable)
		.Entry(Item)
		.SourceCodeAccessor(OnAccessSourceCode)
		.AssetAccessor(OnAccessAsset);
}

void SWidgetSearchPanel::HandleSelectionChanged(TSharedPtr<FWidgetSearchResultEntry> Item, ESelectInfo::Type SelectType)
{
	if (!Item.IsValid())
	{
		return;
	}

	if (IsAllLiveWidgetsMode())
	{
		// In all-live-widgets mode the found nodes are transient — navigate by live widget pointer
		// so the reflector can locate the widget in whatever tree it currently holds.
		if (OnSelectLiveWidget.IsBound())
		{
			OnSelectLiveWidget.Execute(Item->Node->GetLiveWidget());
		}
	}
	else
	{
		// In tree-root mode (searching in the current picked widgets) the found nodes are the same instance objects as in the reflector tree.
		if (OnSelectNode.IsBound())
		{
			OnSelectNode.Execute(Item->Node);
		}
	}
}

} // namespace UE::SlateReflector

#undef LOCTEXT_NAMESPACE
