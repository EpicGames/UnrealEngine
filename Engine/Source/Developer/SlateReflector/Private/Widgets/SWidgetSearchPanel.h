// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Application/IWidgetReflector.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class FWidgetReflectorNodeBase;
class SEditableTextBox;

namespace UE::SlateReflector
{
struct FWidgetSearchResultEntry;

/**
 * A panel that allows searching the widget hierarchy by type, description, tag, and address.
 * Displays matching widgets in a list view; selecting an entry focuses it in the Widget Hierarchy.
 *
 * Two search scopes are available via a checkbox:
 *  - Current tree root  : searches the nodes currently shown in the Widget Hierarchy.
 *  - All live widgets   : builds a transient FWidgetReflectorNodeBase tree from every visible
 *                         SWindow so that widgets outside the current pick scope can be found.
 */
class SWidgetSearchPanel : public SCompoundWidget
{
public:
	/** Called when the user selects a result that belongs to the current tree root. */
	DECLARE_DELEGATE_OneParam(FOnSelectNode, TSharedRef<FWidgetReflectorNodeBase>);

	/** Called when the user selects a result in "all live widgets" mode; passes the live SWidget
	 *  so the hierarchy can locate it even if the tree root hasn't been set for that window yet. */
	DECLARE_DELEGATE_OneParam(FOnSelectLiveWidget, TSharedPtr<const SWidget>);

	/** Called to obtain the current root nodes (live or snapshot) at the time of search. */
	DECLARE_DELEGATE_RetVal(TArray<TSharedRef<FWidgetReflectorNodeBase>>, FGetRootNodes);

	SLATE_BEGIN_ARGS(SWidgetSearchPanel) {}
		SLATE_EVENT(FOnSelectNode,       OnSelectNode)
		SLATE_EVENT(FOnSelectLiveWidget, OnSelectLiveWidget)
		SLATE_EVENT(FGetRootNodes,       OnGetRootNodes)
		SLATE_EVENT(FAccessSourceCode,   OnAccessSourceCode)
		SLATE_EVENT(FAccessAsset,        OnAccessAsset)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	struct FSearchFilters
	{
		FString DisplayName;
		FString Type;
		FString Tag;
		FString Address;

		bool IsEmpty() const
		{
			return Type.IsEmpty() && DisplayName.IsEmpty() && Tag.IsEmpty() && Address.IsEmpty();
		}
	};

	FReply HandleSearchClicked();

	/** Build search roots: either the current tree root or a fresh live tree from all windows. */
	TArray<TSharedRef<FWidgetReflectorNodeBase>> BuildSearchRoots() const;

	void SearchNodes(const TArray<TSharedRef<FWidgetReflectorNodeBase>>& InNodes, const FSearchFilters& Filters);
	bool NodeMatchesFilters(const TSharedRef<FWidgetReflectorNodeBase>& Node, const FSearchFilters& Filters) const;

	TSharedRef<ITableRow> HandleGenerateRow(TSharedPtr<FWidgetSearchResultEntry> Item, const TSharedRef<STableViewBase>& OwnerTable);
	void HandleSelectionChanged(TSharedPtr<FWidgetSearchResultEntry> Item, ESelectInfo::Type SelectType);

	/** Returns true if the "Search all live widgets" checkbox is currently checked. */
	bool IsAllLiveWidgetsMode() const;

	TSharedPtr<SEditableTextBox> TypeSearchBox;
	TSharedPtr<SEditableTextBox> DescSearchBox;
	TSharedPtr<SEditableTextBox> TagSearchBox;
	TSharedPtr<SEditableTextBox> AddressSearchBox;

	TSharedPtr<SCheckBox> AllLiveWidgetsCheckBox;

	TArray<TSharedRef<FWidgetReflectorNodeBase>> LastSearchRoots;

	TArray<TSharedPtr<FWidgetSearchResultEntry>> SearchResults;
	TSharedPtr<SListView<TSharedPtr<FWidgetSearchResultEntry>>> ResultListView;

	FOnSelectNode OnSelectNode;
	FOnSelectLiveWidget OnSelectLiveWidget;
	FGetRootNodes     OnGetRootNodes;
	FAccessSourceCode OnAccessSourceCode;
	FAccessAsset      OnAccessAsset;
};

} // namespace UE::SlateReflector
