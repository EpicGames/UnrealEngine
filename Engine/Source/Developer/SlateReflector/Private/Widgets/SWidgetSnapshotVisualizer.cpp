// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SWidgetSnapshotVisualizer.h"
#include "HAL/FileManager.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/ArrayWriter.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SNavigationSimulationList.h"
#include "Framework/Layout/ScrollyZoomy.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/Base64.h"
#include "Misc/Compression.h"

#include "SlateNavigationEventSimulator.h"
#include "SlateReflectorModule.h"
#include "Styling/WidgetReflectorStyle.h"

#if SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM
#include "DesktopPlatformModule.h"
#endif // SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM

#define LOCTEXT_NAMESPACE "WidgetSnapshotVisualizer"

class SScrollableSnapshotImage : public SCompoundWidget, public IScrollableZoomable
{
	using Super = SCompoundWidget; //SPanel
public:

	SLATE_BEGIN_ARGS(SScrollableSnapshotImage)
		: _SnapshotData(nullptr)
		{
			_Visibility = EVisibility::Visible;
		}
		
		SLATE_ARGUMENT(const FWidgetSnapshotData*, SnapshotData)

		SLATE_EVENT(SWidgetSnapshotVisualizer::FOnWidgetPathPicked, OnWidgetPathPicked);

	SLATE_END_ARGS()

	SScrollableSnapshotImage()
		: PhysicalOffset(ForceInitToZero)
		, CachedSize(ForceInitToZero)
		, ScrollyZoomy(false)
		, SnapshotDataPtr(nullptr)
		, bIsPicking(false)
	{
	}

	void Construct(const FArguments& InArgs)
	{
		SnapshotDataPtr = InArgs._SnapshotData;
		check(SnapshotDataPtr);

		SelectedWindowIndex = INDEX_NONE;

		OnWidgetPathPicked = InArgs._OnWidgetPathPicked;

		ChildSlot
		[
			SNew(SImage)
			.Image(this, &SScrollableSnapshotImage::GetSelectedWindowTextureBrush)
		];
	}

	void SetSelectedWindowIndex(const int32 InIndex)
	{
		SelectedWindowIndex = InIndex;
		PickedWidgets.Reset();
		PhysicalOffset = FVector2f::ZeroVector;
	}

	int32 GetSelectedWindowIndex() const
	{
		return SelectedWindowIndex;
	}

	const FSlateBrush* GetSelectedWindowTextureBrush() const
	{
		return SnapshotDataPtr->GetBrush(SelectedWindowIndex);
	}

	void SetNavigationSimulation(TSharedPtr<SNavigationSimulationSnapshotList> InNavigationSimulationOverlay)
	{
		NavigationSimulationOverlay = InNavigationSimulationOverlay;
	}

	void SetIsPicking(const bool InIsPicking)
	{
		bIsPicking = InIsPicking;
	}

	bool GetIsPicking() const
	{
		return bIsPicking;
	}

	void SetSelectedWidgets(const TArray<TSharedRef<FWidgetReflectorNodeBase>>& InSelectedWidgets)
	{
		SelectedWidgets = InSelectedWidgets;
	}

	virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override
	{
		CachedSize = AllottedGeometry.GetLocalSize();

		const TSharedRef<SWidget>& ChildWidget = ChildSlot.GetWidget();
		if (ChildWidget->GetVisibility() != EVisibility::Collapsed)
		{
			const FVector2f& WidgetDesiredSize = ChildWidget->GetDesiredSize();

			// Clamp the pan offset based on our current geometry
			SScrollableSnapshotImage* const NonConstThis = const_cast<SScrollableSnapshotImage*>(this);
			NonConstThis->ClampViewOffset(WidgetDesiredSize, CachedSize);

			ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(ChildWidget, PhysicalOffset, WidgetDesiredSize));
		}
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		ScrollyZoomy.Tick(InDeltaTime, *this);
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return ScrollyZoomy.OnMouseButtonDown(MouseEvent);
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return ScrollyZoomy.OnMouseButtonUp(AsShared(), MyGeometry, MouseEvent);
	}
		
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		struct FWidgetPicker
		{
			static bool FindWidgetsUnderPoint(const FVector2f& InHitTestPoint, const FVector2f& InWindowPosition, const TSharedRef<FWidgetReflectorNodeBase>& InWidget, TArray<TSharedRef<FWidgetReflectorNodeBase>>& OutWidgets)
			{
				const bool bNeedsHitTesting = InWidget->GetHitTestInfo().IsHitTestVisible || InWidget->GetHitTestInfo().AreChildrenHitTestVisible;
				if (bNeedsHitTesting)
				{
					const FSlateRect HitTestRect = FSlateRect::FromPointAndExtent(
						InWidget->GetAccumulatedLayoutTransform().GetTranslation() - InWindowPosition, 
						TransformPoint(InWidget->GetAccumulatedLayoutTransform().GetScale(), InWidget->GetLocalSize())
						);

					if (HitTestRect.ContainsPoint(InHitTestPoint))
					{
						OutWidgets.Add(InWidget);

						if (InWidget->GetHitTestInfo().AreChildrenHitTestVisible)
						{
							for (const auto& ChildWidget : InWidget->GetChildNodes())
							{
								if (FindWidgetsUnderPoint(InHitTestPoint, InWindowPosition, ChildWidget, OutWidgets))
								{
									return true;
								}
							}
						}

						return InWidget->GetHitTestInfo().IsHitTestVisible;
					}
				}

				return false;
			}
		};

		if (bIsPicking)
		{
			// We need to pick in the snapshot window space, so convert the mouse co-ordinates to be relative to our top-left position
			const FVector2f& ScreenMousePos = MouseEvent.GetScreenSpacePosition();
			const FVector2f LocalMousePos = MyGeometry.AbsoluteToLocal(ScreenMousePos);
			const FVector2f ScrolledPos = LocalMousePos - PhysicalOffset;

			PickedWidgets.Reset();

			TSharedPtr<FWidgetReflectorNodeBase> Window = SnapshotDataPtr->GetWindow(SelectedWindowIndex);
			if (Window.IsValid())
			{
				FWidgetPicker::FindWidgetsUnderPoint(
					ScrolledPos, 
					Window->GetAccumulatedLayoutTransform().GetTranslation(), 
					Window.ToSharedRef(), 
					PickedWidgets
					);
			}

			if (PickedWidgets.Num() > 0)
			{
				OnWidgetPathPicked.ExecuteIfBound(PickedWidgets);
			}
		}

		return ScrollyZoomy.OnMouseMove(AsShared(), *this, MyGeometry, MouseEvent);
	}

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		ScrollyZoomy.OnMouseLeave(AsShared(), MouseEvent);
	}
		
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return ScrollyZoomy.OnMouseWheel(MouseEvent, *this);
	}

	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override
	{
		FCursorReply Reply = ScrollyZoomy.OnCursorQuery();
		
		if (!Reply.IsEventHandled() && !bIsPicking)
		{
			Reply = FCursorReply::Cursor(EMouseCursor::GrabHand);
		}

		return Reply;
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		FSlateClippingZone ClippingZone(AllottedGeometry);
		OutDrawElements.PushClip(ClippingZone);

		LayerId = Super::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
		LayerId = ScrollyZoomy.PaintSoftwareCursorIfNeeded(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);

		TSharedPtr<FWidgetReflectorNodeBase> Window = SnapshotDataPtr->GetWindow(SelectedWindowIndex);
		if (Window.IsValid())
		{
			static const FName DebugBorderBrush = TEXT("Debug.Border");
			const FVector2D RootDrawOffset = PhysicalOffset - Window->GetAccumulatedLayoutTransform().GetTranslation();
			const FSlateBrush* Brush = FCoreStyle::Get().GetBrush(DebugBorderBrush);
			if (bIsPicking)
			{
				const FLinearColor TopmostWidgetColor(1.0f, 0.0f, 0.0f);
				const FLinearColor LeafmostWidgetColor(0.0f, 1.0f, 0.0f);

				for (int32 WidgetIndex = 0; WidgetIndex < PickedWidgets.Num(); ++WidgetIndex)
				{
					const TSharedRef<FWidgetReflectorNodeBase>& PickedWidget = PickedWidgets[WidgetIndex];
					const float ColorFactor = static_cast<float>(WidgetIndex)/ static_cast<float>(PickedWidgets.Num());
					const FLinearColor Tint(1.0f - ColorFactor, ColorFactor, 0.0f, 1.0f);

					FSlateDrawElement::MakeBox(
						OutDrawElements,
						++LayerId,
						AllottedGeometry.ToPaintGeometry(TransformPoint(PickedWidget->GetAccumulatedLayoutTransform().GetScale(), PickedWidget->GetLocalSize()), FSlateLayoutTransform(RootDrawOffset + PickedWidget->GetAccumulatedLayoutTransform().GetTranslation())),
						Brush,
						ESlateDrawEffect::None,
						FMath::Lerp(TopmostWidgetColor, LeafmostWidgetColor, ColorFactor)
					);
				}
			}
			else
			{
				for (const auto& SelectedWidget : SelectedWidgets)
				{
					FSlateDrawElement::MakeBox(
						OutDrawElements,
						++LayerId,
						AllottedGeometry.ToPaintGeometry(TransformPoint(SelectedWidget->GetAccumulatedLayoutTransform().GetScale(), SelectedWidget->GetLocalSize()), FSlateLayoutTransform(RootDrawOffset + SelectedWidget->GetAccumulatedLayoutTransform().GetTranslation())),
						Brush,
						ESlateDrawEffect::None,
						SelectedWidget->GetTint()
					);
				}
			}

			if (NavigationSimulationOverlay)
			{
				NavigationSimulationOverlay->PaintNodesWithOffset(AllottedGeometry, OutDrawElements, LayerId, RootDrawOffset);
			}
		}

		OutDrawElements.PopClip();
		return LayerId;
	}

	virtual bool ScrollBy(const FVector2D& Offset) override
	{
		const FVector2f PrevPhysicalOffset = PhysicalOffset;
		PhysicalOffset += UE::Slate::CastToVector2f(Offset);

		const TSharedRef<SWidget>& ChildWidget = ChildSlot.GetWidget();
		const FVector2f& WidgetDesiredSize = ChildWidget->GetDesiredSize();
		ClampViewOffset(WidgetDesiredSize, CachedSize);

		return PhysicalOffset != PrevPhysicalOffset;
	}

	virtual bool ZoomBy(const float Amount) override
	{
		return false;
	}

	float GetZoomLevel() const
	{
		return 1.0f;
	}

private:
	void ClampViewOffset(const FVector2f& ViewportSize, const FVector2f& LocalSize)
	{
		PhysicalOffset.X = ClampViewOffsetAxis(ViewportSize.X, LocalSize.X, PhysicalOffset.X);
		PhysicalOffset.Y = ClampViewOffsetAxis(ViewportSize.Y, LocalSize.Y, PhysicalOffset.Y);
	}

	float ClampViewOffsetAxis(const float ViewportSize, const float LocalSize, const float CurrentOffset)
	{
		if (ViewportSize <= LocalSize)
		{
			// If the viewport is smaller than the available size, then we can't be scrolled
			return 0.0f;
		}

		// Given the size of the viewport, and the current size of the window, work how far we can scroll
		// Note: This number is negative since scrolling down/right moves the viewport up/left
		const float MaxScrollOffset = LocalSize - ViewportSize;

		// Clamp the left/top edge
		if (CurrentOffset < MaxScrollOffset)
		{
			return MaxScrollOffset;
		}

		// Clamp the right/bottom edge
		if (CurrentOffset > 0.0f)
		{
			return 0.0f;
		}

		return CurrentOffset;
	}

	FVector2f PhysicalOffset;
	mutable FVector2f CachedSize;

	FScrollyZoomy ScrollyZoomy;

	/** Snapshot data we're visualizing */
	const FWidgetSnapshotData* SnapshotDataPtr;

	/** Index of the window we're currently viewing */
	int32 SelectedWindowIndex;

	SWidgetSnapshotVisualizer::FOnWidgetPathPicked OnWidgetPathPicked;

	bool bIsPicking;
	TArray<TSharedRef<FWidgetReflectorNodeBase>> PickedWidgets;

	TArray<TSharedRef<FWidgetReflectorNodeBase>> SelectedWidgets;

	TSharedPtr<SNavigationSimulationSnapshotList> NavigationSimulationOverlay;
};


FWidgetSnapshotData::~FWidgetSnapshotData()
{
	DestroyBrushes();
}

void FWidgetSnapshotData::ClearSnapshot()
{
	Reset();
}

void FWidgetSnapshotData::TakeSnapshot(bool bSimulateNavigation)
{
	TArray<TSharedRef<SWindow>> VisibleWindows;
	FSlateApplication::Get().GetAllVisibleWindowsOrdered(VisibleWindows);

	CreateSnapshot(VisibleWindows, bSimulateNavigation);
}

void FWidgetSnapshotData::CreateSnapshot(const TArray<TSharedRef<SWindow>>& VisibleWindows, bool bSimulateNavigation)
{
	Reset();
	Reserve(VisibleWindows.Num());

	for (const TSharedRef<SWindow>& VisibleWindow : VisibleWindows)
	{
		// Snapshot the current state of this window widget hierarchy
		Windows.Add(FWidgetReflectorNodeUtils::NewSnapshotNodeTreeFrom(FArrangedWidget(VisibleWindow, VisibleWindow->GetWindowGeometryInScreen())));

		if (bSimulateNavigation)
		{
			TArray<FSlateNavigationEventSimulator::FSimulationResult> NavigationSimulation;
			NavigationSimulation = FSlateReflectorModule::GetModulePtr()->GetNavigationEventSimulator()->SimulateForEachWidgets(VisibleWindow, 0, ENavigationGenesis::Controller, FSlateNavigationEventSimulator::ENavigationStyle::FourCardinalDirections);
			FWidgetSnapshotNavigationSimulationData SimulationData;
			SimulationData.SimulationData = FNavigationSimulationNodeUtils::BuildNavigationSimulationNodeListForSnapshot(NavigationSimulation);
			NavigationSimulationData.Add(MoveTemp(SimulationData));
		}

		// Screenshot the current window so we can pick against its current state
		FWidgetSnapshotTextureData& TextureData = WindowTextureData[WindowTextureData.AddDefaulted()];
		FSlateApplication::Get().TakeScreenshot(VisibleWindow, TextureData.ColorData, TextureData.Dimensions);
	}

	CreateBrushes();
}

bool FWidgetSnapshotData::SaveSnapshotToFile(const FString& InFilename) const
{
	TSharedRef<FJsonObject> RootJsonObject = SaveSnapshotAsJson();

	FArchive* const FileAr = IFileManager::Get().CreateFileWriter(*InFilename);
	if (FileAr)
	{
		typedef TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>> FStringWriter;
		typedef TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>> FStringWriterFactory;

		TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(FileAr);
		FJsonSerializer::Serialize(RootJsonObject, Writer);
		FileAr->Close();
		return true;
	}

	return false;
}

void FWidgetSnapshotData::SaveSnapshotToBuffer(TArray<uint8>& OutData) const
{
	TSharedRef<FJsonObject> RootJsonObject = SaveSnapshotAsJson();

	typedef TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>> FStringWriter;
	typedef TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>> FStringWriterFactory;

	FArrayWriter TmpJsonData;
	TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(&TmpJsonData);
	FJsonSerializer::Serialize(RootJsonObject, Writer);

	OutData.Reset();
	FMemoryWriter BufferWriter(OutData);

	int32 UncompressedDataSize = TmpJsonData.Num();
	BufferWriter << UncompressedDataSize;

	BufferWriter.SerializeCompressed(TmpJsonData.GetData(), TmpJsonData.Num(), NAME_Zlib);
}

double SnapshotJsonVersion = 2.2;

TSharedRef<FJsonObject> FWidgetSnapshotData::SaveSnapshotAsJson() const
{
	check(Windows.Num() == WindowTextureData.Num());

	TSharedRef<FJsonObject> RootJsonObject = MakeShared<FJsonObject>();

	{
		RootJsonObject->SetNumberField(TEXT("Version"), SnapshotJsonVersion);
	}

	{
		TArray<TSharedPtr<FJsonValue>> WindowsJsonArray;
		for (const TSharedPtr<FWidgetReflectorNodeBase>& Window : Windows)
		{
			check(Window->GetNodeType() == EWidgetReflectorNodeType::Snapshot);
			WindowsJsonArray.Add(FSnapshotWidgetReflectorNode::ToJson(StaticCastSharedRef<FSnapshotWidgetReflectorNode>(Window.ToSharedRef())));
		}
		RootJsonObject->SetArrayField(TEXT("Windows"), WindowsJsonArray);
	}

	{
		TArray<TSharedPtr<FJsonValue>> NavigationDataArray;
		
		for (const FWidgetSnapshotNavigationSimulationData& NavigationSimulation : NavigationSimulationData)
		{
			TSharedRef<FJsonObject> NavigationData = MakeShared<FJsonObject>();
			TArray<TSharedPtr<FJsonValue>> SimulationDataArray;
			for (const FNavigationSimulationWidgetNodePtr& SimulationData : NavigationSimulation.SimulationData)
			{
				check(SimulationData.Get());
				SimulationDataArray.Add(FNavigationSimulationWidgetNode::ToJson(*SimulationData.Get()));
			}
			NavigationData->SetArrayField(TEXT("SimulationData"), SimulationDataArray);
			NavigationDataArray.Add(MakeShared<FJsonValueObject>(NavigationData));
		}
		RootJsonObject->SetArrayField(TEXT("NavigationData"), NavigationDataArray);
	}

	{
		TArray<TSharedPtr<FJsonValue>> TexturesJsonArray;
		for (const FWidgetSnapshotTextureData& TextureData : WindowTextureData)
		{
			TSharedRef<FJsonObject> TextureDataJsonObject = MakeShareable(new FJsonObject());

			{
				TArray<TSharedPtr<FJsonValue>> StructJsonArray;
				StructJsonArray.Add(MakeShareable(new FJsonValueNumber(TextureData.Dimensions.X)));
				StructJsonArray.Add(MakeShareable(new FJsonValueNumber(TextureData.Dimensions.Y)));
				TextureDataJsonObject->SetArrayField(TEXT("Dimensions"), StructJsonArray);
			}

			{
				// This is raw texture data - compress it before we encode it to save space
				const int32 UncompressedDataSizeBytes = TextureData.ColorData.Num() * sizeof(FColor);
				TArray<uint8> CompressedDataBuffer;
				CompressedDataBuffer.AddZeroed(FCompression::CompressMemoryBound(NAME_Zlib, UncompressedDataSizeBytes));
				int32 CompressedDataSize = CompressedDataBuffer.Num();
				if (FCompression::CompressMemory(NAME_Zlib, CompressedDataBuffer.GetData(), CompressedDataSize, TextureData.ColorData.GetData(), UncompressedDataSizeBytes))
				{
					TextureDataJsonObject->SetBoolField(TEXT("IsCompressed"), true);
					TextureDataJsonObject->SetNumberField(TEXT("UncompressedSize"), UncompressedDataSizeBytes);

					// FCompression::CompressMemory updates CompressedDataSize with the actual size - we may have to shrink our buffer count now
					CompressedDataBuffer.SetNum(CompressedDataSize, EAllowShrinking::No);

					const FString EncodedTextureData = FBase64::Encode(CompressedDataBuffer);
					TextureDataJsonObject->SetStringField(TEXT("TextureData"), EncodedTextureData);
				}
				else
				{
					TextureDataJsonObject->SetBoolField(TEXT("IsCompressed"), false);

					// Failed to compress... use the raw texture data
					TArray<uint8> TextureDataBytes;
					TextureDataBytes.Append(reinterpret_cast<const uint8*>(TextureData.ColorData.GetData()), TextureData.ColorData.Num() * sizeof(FColor));

					const FString EncodedTextureData = FBase64::Encode(TextureDataBytes);
					TextureDataJsonObject->SetStringField(TEXT("TextureData"), EncodedTextureData);
				}
			}

			TexturesJsonArray.Add(MakeShareable(new FJsonValueObject(TextureDataJsonObject)));
		}
		RootJsonObject->SetArrayField(TEXT("Textures"), TexturesJsonArray);
	}

	return RootJsonObject;
}

bool FWidgetSnapshotData::LoadSnapshotFromFile(const FString& InFilename)
{
	bool bJsonLoaded = false;
	TSharedPtr<FJsonObject> RootJsonObject;

	{
		FArchive* FileAr = IFileManager::Get().CreateFileReader(*InFilename);
		if (FileAr)
		{
			typedef TJsonReader<TCHAR> FJsonReader;
			typedef TJsonReaderFactory<TCHAR> FJsonReaderFactory;

			TSharedRef<FJsonReader> Reader = FJsonReaderFactory::Create(FileAr);
			bJsonLoaded = FJsonSerializer::Deserialize(Reader, RootJsonObject);
			FileAr->Close();
			FileAr = nullptr;
		}
	}

	if (bJsonLoaded)
	{
		check(RootJsonObject.IsValid());
		LoadSnapshotFromJson(RootJsonObject.ToSharedRef());
		return true;
	}

	return false;
}

void FWidgetSnapshotData::LoadSnapshotFromBuffer(const TArray<uint8>& InData)
{
	int32 UncompressedDataSize = 0;
	TArray<uint8> UncompressedData;

	{
		FMemoryReader BufferReader(InData);

		BufferReader << UncompressedDataSize;

		UncompressedData.AddZeroed(UncompressedDataSize);
		BufferReader.SerializeCompressed(UncompressedData.GetData(), UncompressedDataSize, NAME_Zlib);
	}

	bool bJsonLoaded = false;
	TSharedPtr<FJsonObject> RootJsonObject;

	if (UncompressedData.Num() > 0)
	{
		typedef TJsonReader<TCHAR> FJsonReader;
		typedef TJsonReaderFactory<TCHAR> FJsonReaderFactory;

		FMemoryReader UncompressedDataReader(UncompressedData);
		TSharedRef<FJsonReader> Reader = FJsonReaderFactory::Create(&UncompressedDataReader);
		bJsonLoaded = FJsonSerializer::Deserialize(Reader, RootJsonObject);
	}

	if (bJsonLoaded)
	{
		check(RootJsonObject.IsValid());
		LoadSnapshotFromJson(RootJsonObject.ToSharedRef());
	}
}

void FWidgetSnapshotData::LoadSnapshotFromJson(const TSharedRef<FJsonObject>& InRootJsonObject)
{
	Reset();

	{
		const double VersionNumber = InRootJsonObject->GetNumberField(TEXT("Version"));
		if (VersionNumber < SnapshotJsonVersion)
		{
			UE_LOGF(LogSlate, Error, "The version of the snapshot (%f) is older than the current version (%f). New fields will be initialized to their defaulted value.", VersionNumber, SnapshotJsonVersion);
		}
	}

	{
		const TArray<TSharedPtr<FJsonValue>>& WindowsJsonArray = InRootJsonObject->GetArrayField(TEXT("Windows"));
		for (const TSharedPtr<FJsonValue>& WindowJsonValue : WindowsJsonArray)
		{
			Windows.Add(FSnapshotWidgetReflectorNode::FromJson(WindowJsonValue.ToSharedRef()));
		}
	}

	{
		const TArray<TSharedPtr<FJsonValue>>& NavigationDataJsonArray = InRootJsonObject->GetArrayField(TEXT("NavigationData"));
		for (const TSharedPtr<FJsonValue>& NavigationDataJsonValue : NavigationDataJsonArray)
		{
			FWidgetSnapshotNavigationSimulationData NavigationData;
			const TSharedPtr<FJsonObject>& NavigationDataJsonObject = NavigationDataJsonValue->AsObject();
			const TArray<TSharedPtr<FJsonValue>>& SimulationDataJsonArray = NavigationDataJsonObject->GetArrayField(TEXT("SimulationData"));
			for (const TSharedPtr<FJsonValue>& SimulationDataJsonValue : SimulationDataJsonArray)
			{
				FNavigationSimulationWidgetNodePtr SimulationData = FNavigationSimulationWidgetNode::FromJson(SimulationDataJsonValue.ToSharedRef());
				NavigationData.SimulationData.Add(SimulationData);
			}
			NavigationSimulationData.Add(MoveTemp(NavigationData));
		}
	}

	{
		const TArray<TSharedPtr<FJsonValue>>& TexturesJsonArray = InRootJsonObject->GetArrayField(TEXT("Textures"));
		for (const TSharedPtr<FJsonValue>& TextureDataJsonValue : TexturesJsonArray)
		{
			const TSharedPtr<FJsonObject>& TextureDataJsonObject = TextureDataJsonValue->AsObject();
			check(TextureDataJsonObject.IsValid());

			FWidgetSnapshotTextureData TextureData;

			{
				const TArray<TSharedPtr<FJsonValue>>& StructJsonArray = TextureDataJsonObject->GetArrayField(TEXT("Dimensions"));
				check(StructJsonArray.Num() == 2);

				TextureData.Dimensions.X = (int32)(StructJsonArray[0]->AsNumber());
				TextureData.Dimensions.Y = (int32)(StructJsonArray[1]->AsNumber());
			}

			{
				const FString EncodedTextureData = TextureDataJsonObject->GetStringField(TEXT("TextureData"));
				TArray<uint8> DecodedTextureDataBytes;
				FBase64::Decode(EncodedTextureData, DecodedTextureDataBytes);

				const bool bIsCompressed = TextureDataJsonObject->GetBoolField(TEXT("IsCompressed"));
				if (bIsCompressed)
				{
					const int32 UncompressedDataSizeBytes = (int32)(TextureDataJsonObject->GetNumberField(TEXT("UncompressedSize")));
					TextureData.ColorData.AddZeroed(UncompressedDataSizeBytes / sizeof(FColor));

					FCompression::UncompressMemory(NAME_Zlib, TextureData.ColorData.GetData(), UncompressedDataSizeBytes, DecodedTextureDataBytes.GetData(), DecodedTextureDataBytes.Num());
				}
				else
				{
					TextureData.ColorData.Append(reinterpret_cast<const FColor*>(DecodedTextureDataBytes.GetData()), DecodedTextureDataBytes.Num() / sizeof(FColor));
				}
			}

			WindowTextureData.Add(TextureData);
		}
	}

	CreateBrushes();
}

bool FWidgetSnapshotData::IsEmpty() const
{
	return Windows.Num() == 0;
}

int32 FWidgetSnapshotData::Num() const
{
	return Windows.Num();
}

const TArray<TSharedPtr<FWidgetReflectorNodeBase>>& FWidgetSnapshotData::GetWindowsPtr() const
{
	return Windows;
}

TArray<TSharedRef<FWidgetReflectorNodeBase>> FWidgetSnapshotData::GetWindowsRef() const
{
	TArray<TSharedRef<FWidgetReflectorNodeBase>> RetWindows;
	RetWindows.Reserve(Windows.Num());
	for (const auto& Window : Windows)
	{
		RetWindows.Add(Window.ToSharedRef());
	}
	return RetWindows;
}

TSharedPtr<FWidgetReflectorNodeBase> FWidgetSnapshotData::GetWindow(const int32 WindowIndex) const
{
	return (Windows.IsValidIndex(WindowIndex)) 
		? TSharedPtr<FWidgetReflectorNodeBase>(Windows[WindowIndex]) 
		: TSharedPtr<FWidgetReflectorNodeBase>(nullptr);
}

const FWidgetSnapshotNavigationSimulationData& FWidgetSnapshotData::GetNavigationSimulation(const int32 WindowIndex) const
{
	return (NavigationSimulationData.IsValidIndex(WindowIndex))
		? NavigationSimulationData[WindowIndex]
		: EmptyNavigationSimulationData;
}

const FSlateBrush* FWidgetSnapshotData::GetBrush(const int32 WindowIndex) const
{
	return (WindowTextureBrushes.IsValidIndex(WindowIndex)) ? WindowTextureBrushes[WindowIndex].Get() : nullptr;
}

void FWidgetSnapshotData::CreateBrushes()
{
	DestroyBrushes();

	WindowTextureBrushes.Reserve(WindowTextureData.Num());

	static int32 TextureIndex = 0;
	for (const FWidgetSnapshotTextureData& TextureData : WindowTextureData)
	{
		if (TextureData.ColorData.Num() > 0)
		{
			TArray<uint8> TextureDataAsBGRABytes;
			TextureDataAsBGRABytes.Reserve(TextureData.ColorData.Num() * 4);
			for (const FColor& PixelColor : TextureData.ColorData)
			{
				TextureDataAsBGRABytes.Add(PixelColor.B);
				TextureDataAsBGRABytes.Add(PixelColor.G);
				TextureDataAsBGRABytes.Add(PixelColor.R);
				TextureDataAsBGRABytes.Add(PixelColor.A);
			}

			WindowTextureBrushes.Add(FSlateDynamicImageBrush::CreateWithImageData(
				*FString::Printf(TEXT("FWidgetSnapshotData_WindowTextureBrush_%d"), TextureIndex++), 
				FVector2D((float)TextureData.Dimensions.X, (float)TextureData.Dimensions.Y),
				TextureDataAsBGRABytes
				));
		}
		else
		{
			WindowTextureBrushes.Add(nullptr);
		}
	}
}

void FWidgetSnapshotData::DestroyBrushes()
{
	for (const auto& WindowTextureBrush : WindowTextureBrushes)
	{
		if (WindowTextureBrush.IsValid())
		{
			WindowTextureBrush->ReleaseResource();
		}
	}

	WindowTextureBrushes.Reset();
}

void FWidgetSnapshotData::Reserve(const int32 NumWindows)
{
	Windows.Reserve(NumWindows);
	NavigationSimulationData.Reserve(NumWindows);
	WindowTextureData.Reserve(NumWindows);
	WindowTextureBrushes.Reserve(NumWindows);
}

void FWidgetSnapshotData::Reset()
{
	DestroyBrushes();

	Windows.Reset();
	NavigationSimulationData.Reset();
	WindowTextureData.Reset();
	WindowTextureBrushes.Reset();
}


void SWidgetSnapshotVisualizer::Construct(const FArguments& InArgs)
{
	SnapshotDataPtr = InArgs._SnapshotData;
	check(SnapshotDataPtr);
	FSlimHorizontalToolBarBuilder ToolbarBuilderGlobal(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);
	ToolbarBuilderGlobal.SetStyle(&FAppStyle::Get(), "SlimToolBar");
	SAssignNew(WindowPickerCombo, SComboBox<TSharedPtr<FWidgetReflectorNodeBase>>)
		.OptionsSource(&SnapshotDataPtr->GetWindowsPtr())
		.OnSelectionChanged(this, &SWidgetSnapshotVisualizer::OnWindowSelectionChanged)
		.OnGenerateWidget(this, &SWidgetSnapshotVisualizer::GenerateWindowPickerComboItem)
		[
			SNew(STextBlock)
			.Text(this, &SWidgetSnapshotVisualizer::GetSelectedWindowComboItemText)
		]
		.IsEnabled(this, &SWidgetSnapshotVisualizer::HasValidSnapshot);

	ToolbarBuilderGlobal.BeginSection("Picking");
	{
		
		FTextBuilder TooltipText;
		ToolbarBuilderGlobal.AddWidget(WindowPickerCombo.ToSharedRef());
		ToolbarBuilderGlobal.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateSP(this, &SWidgetSnapshotVisualizer::OnPickWidgetClicked),
				FCanExecuteAction::CreateSP(this, &SWidgetSnapshotVisualizer::HasValidSnapshot),
				FGetActionCheckState::CreateSP(this, &SWidgetSnapshotVisualizer::GetPickWidgetColor)
			),
			NAME_None,
			MakeAttributeSP(this, &SWidgetSnapshotVisualizer::GetPickWidgetText),
			TooltipText.ToText(),
			FSlateIcon(FWidgetReflectorStyle::GetStyleSetName(), "Icon.HitTestPicking"),
			EUserInterfaceActionType::ToggleButton
		);
#if SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM
		
		ToolbarBuilderGlobal.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateSP(this, &SWidgetSnapshotVisualizer::OnSaveSnapshotClicked),
				FCanExecuteAction::CreateSP(this, &SWidgetSnapshotVisualizer::HasValidSnapshot),
				FGetActionCheckState()
			),
			NAME_None,
			LOCTEXT("SaveSnapshotButtonText", "Save Snapshot"),
			TooltipText.ToText(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save"),
			EUserInterfaceActionType::Button
		);
#endif


	}
	ToolbarBuilderGlobal.EndSection();
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		.BorderBackgroundColor(FLinearColor::Gray) // Darken the outer border
		.Padding(2.0f)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.f)
			[
				ToolbarBuilderGlobal.MakeWidget()

			]

			+SVerticalBox::Slot()
			.Padding(2.f)
			[
				SNew(SSplitter)
				.Orientation(EOrientation::Orient_Horizontal)

				+ SSplitter::Slot()
				[
					SNew(SBorder)
					.Padding(2.0f)
					.BorderImage(FCoreStyle::Get().GetBrush(TEXT("FocusRectangle")))
					[
						SAssignNew(SnapshotImage, SScrollableSnapshotImage)
						.SnapshotData(InArgs._SnapshotData)
						.OnWidgetPathPicked(InArgs._OnWidgetPathPicked)
					]
				]

				+ SSplitter::Slot()
				.SizeRule(SSplitter::SizeToContent)
				[
					SAssignNew(NavigationSimulationList, SNavigationSimulationSnapshotList, InArgs._OnSnapshotWidgetSelected, InArgs._OnSnapshotWidgetSelected)
					.ListItemsSource(&SnapshotDataPtr->GetNavigationSimulation(0).SimulationData)
					.Visibility(this, &SWidgetSnapshotVisualizer::HandleGetNavigationSimulationListVisibility)
				]
			]
		]
	];

	SnapshotImage->SetNavigationSimulation(NavigationSimulationList);

	SnapshotDataUpdated();
}

void SWidgetSnapshotVisualizer::SnapshotDataUpdated()
{
	if (SnapshotImage.IsValid())
	{
		SnapshotImage->SetSelectedWindowIndex(0);
	}

	if (WindowPickerCombo.IsValid())
	{
		WindowPickerCombo->RefreshOptions();
		WindowPickerCombo->SetSelectedItem(SnapshotDataPtr->GetWindow(0));
	}

	if (NavigationSimulationList.IsValid())
	{
		NavigationSimulationList->SetListItemsSource(SnapshotDataPtr->GetNavigationSimulation(0).SimulationData);
	}
}

void SWidgetSnapshotVisualizer::SetSelectedWidgets(const TArray<TSharedRef<FWidgetReflectorNodeBase>>& InSelectedWidgets)
{
	if (SnapshotImage.IsValid())
	{
		SnapshotImage->SetSelectedWidgets(InSelectedWidgets);
	}
	if (NavigationSimulationList)
	{
		FNavigationSimulationWidgetInfo::TPointerAsInt PointerAsInt = InSelectedWidgets.Num() > 0 ? InSelectedWidgets.Last()->GetWidgetAddress() : 0;
		NavigationSimulationList->SelectSnapshotWidget(PointerAsInt);
	}
}

FReply SWidgetSnapshotVisualizer::OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (SnapshotImage.IsValid() && InKeyEvent.GetKey() == EKeys::Escape)
	{
		SnapshotImage->SetIsPicking(false);
	}

	return FReply::Unhandled();
}

void SWidgetSnapshotVisualizer::OnWindowSelectionChanged(TSharedPtr<FWidgetReflectorNodeBase> InWindow, ESelectInfo::Type InReason)
{
	int32 SelectedWindowIndex = INDEX_NONE;
	SnapshotDataPtr->GetWindowsPtr().Find(InWindow, SelectedWindowIndex);

	if (SnapshotImage.IsValid())
	{
		SnapshotImage->SetSelectedWindowIndex(SelectedWindowIndex);
	}
	if (NavigationSimulationList.IsValid())
	{
		NavigationSimulationList->SetListItemsSource(SnapshotDataPtr->GetNavigationSimulation(SelectedWindowIndex).SimulationData);
	}
}

FText SWidgetSnapshotVisualizer::GetWindowPickerComboItemText(TSharedPtr<FWidgetReflectorNodeBase> InWindow)
{
	return FText::Format(LOCTEXT("WidgetComboItemFmt", "{0} - {1}"), InWindow->GetWidgetType(), InWindow->GetWidgetReadableLocation());
}

FText SWidgetSnapshotVisualizer::GetSelectedWindowComboItemText() const
{
	const int32 SelectedWindowIndex = SnapshotImage.IsValid() ? SnapshotImage->GetSelectedWindowIndex() : INDEX_NONE;
	TSharedPtr<FWidgetReflectorNodeBase> SelectedWindowPtr = SnapshotDataPtr->GetWindow(SelectedWindowIndex);
	return (SelectedWindowPtr.IsValid()) ? GetWindowPickerComboItemText(SelectedWindowPtr) : FText::GetEmpty();
}

TSharedRef<SWidget> SWidgetSnapshotVisualizer::GenerateWindowPickerComboItem(TSharedPtr<FWidgetReflectorNodeBase> InWindow) const
{
	return SNew(STextBlock)
		.Text(GetWindowPickerComboItemText(InWindow));
}

FText SWidgetSnapshotVisualizer::GetPickWidgetText() const
{
	const bool bIsPicking = SnapshotImage.IsValid() && SnapshotImage->GetIsPicking();
	return (bIsPicking) ? LOCTEXT("PickingWidget", "Picking (Esc to Stop)") : LOCTEXT("PickSnapshotWidget", "Pick Snapshot Widget");
}

ECheckBoxState SWidgetSnapshotVisualizer::GetPickWidgetColor() const
{
	const bool bIsPicking = SnapshotImage.IsValid() && SnapshotImage->GetIsPicking();
	return bIsPicking
		? ECheckBoxState::Checked
		: ECheckBoxState::Unchecked;
}

void SWidgetSnapshotVisualizer::OnPickWidgetClicked()
{
	if (SnapshotImage.IsValid())
	{
		SnapshotImage->SetIsPicking(!SnapshotImage->GetIsPicking());
	}

}

bool SWidgetSnapshotVisualizer::HasValidSnapshot() const
{
	return SnapshotDataPtr && !SnapshotDataPtr->IsEmpty();
}

EVisibility SWidgetSnapshotVisualizer::HandleGetNavigationSimulationListVisibility() const
{
	if (SnapshotDataPtr && SnapshotDataPtr->GetNavigationSimulation(0).SimulationData.Num() != 0)
	{
		return EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

#if SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM

void SWidgetSnapshotVisualizer::OnSaveSnapshotClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

	if (DesktopPlatform)
	{
		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this));

		TArray<FString> SaveFilenames;
		const bool bOpened = DesktopPlatform->SaveFileDialog(
			(ParentWindow.IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr,
			LOCTEXT("SaveSnapshotDialogTitle", "Save Widget Snapshot").ToString(),
			FPaths::GameAgnosticSavedDir(),
			TEXT(""),
			TEXT("Slate Widget Snapshot (*.widgetsnapshot)|*.widgetsnapshot"),
			EFileDialogFlags::None,
			SaveFilenames
			);

		if (SaveFilenames.Num() > 0)
		{
			SnapshotDataPtr->SaveSnapshotToFile(SaveFilenames[0]);
		}
	}


}

#endif // SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM

#undef LOCTEXT_NAMESPACE
