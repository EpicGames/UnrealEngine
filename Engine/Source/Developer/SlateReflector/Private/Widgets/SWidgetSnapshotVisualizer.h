// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Models/WidgetReflectorNode.h"
#include "Models/NavigationSimulationNode.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SNavigationSimulationList.h"
#include "Widgets/SWindow.h"

class FJsonObject;
class SScrollableSnapshotImage;

/** 
 * The raw texture data from taking a screenshot of a Slate widget (typically the root window)
 */
struct FWidgetSnapshotTextureData
{
	/** The dimensions of the texture */
	FIntVector Dimensions;

	/** The raw color data for the texture (BGRA) */
	TArray<FColor> ColorData;
};


/**
 * The data of a Navigation Event Simulation
 */
struct FWidgetSnapshotNavigationSimulationData
{
	TArray<FNavigationSimulationWidgetNodePtr> SimulationData;
};

/** 
 * All of the data relating to a single widget hierarchy snapshot
 */
class FWidgetSnapshotData
{
public:
	/** Destructor */
	~FWidgetSnapshotData();

	/** Clear the current snapshot data so that we can reclaim the memory */
	void ClearSnapshot();

	/** Take a snapshot of all of the windows that are currently open */
	void TakeSnapshot(bool bSimulateNavigation);

	/** Create a snapshot of the given windows */
	void CreateSnapshot(const TArray<TSharedRef<SWindow>>& VisibleWindows, bool bSimulateNavigation);

	/** Save this snapshot data to the given file. The data will be saved as uncompressed JSON data. */
	bool SaveSnapshotToFile(const FString& InFilename) const;

	/** Save this snapshot data to the given buffer. The data will be saved as zlib compressed JSON data. */
	void SaveSnapshotToBuffer(TArray<uint8>& OutData) const;

	/** Create a JSON object that represents the snapshot data. */
	TSharedRef<FJsonObject> SaveSnapshotAsJson() const;

	/** Populate this snapshot data from the given file. */
	bool LoadSnapshotFromFile(const FString& InFilename);

	/** Populate this snapshot data from the given buffer. */
	void LoadSnapshotFromBuffer(const TArray<uint8>& InData);

	/** Populate this snapshot data from the given JSON object */
	void LoadSnapshotFromJson(const TSharedRef<FJsonObject>& InRootJsonObject);

	/** Check to see whether this snapshot is empty (contains no windows) */
	bool IsEmpty() const;

	/** Get the number of windows this snapshot contains */
	int32 Num() const;

	/** Get the internal windows array */
	const TArray<TSharedPtr<FWidgetReflectorNodeBase>>& GetWindowsPtr() const;

	/** Get the internal windows array, but with each window converted to a TSharedRef */
	TArray<TSharedRef<FWidgetReflectorNodeBase>> GetWindowsRef() const;

	/** Get the window for the given index, or null if the index is invalid */
	TSharedPtr<FWidgetReflectorNodeBase> GetWindow(const int32 WindowIndex) const;

	/** Get the navigation simulation for the given index, or null if the index is invalid or not navigation data was created */
	const FWidgetSnapshotNavigationSimulationData& GetNavigationSimulation(const int32 WindowIndex) const;

	/** Get the brush for the given index, or null if the index is invalid */
	const FSlateBrush* GetBrush(const int32 WindowIndex) const;

private:
	/** Create the dynamic Slate brushes from the texture data for each window */
	void CreateBrushes();

	/** Destroy the dynamic Slate brushes for each window */
	void DestroyBrushes();

	/** Reserve space in all of our internal arrays for the given number of entries */
	void Reserve(const int32 NumWindows);

	/** Reset our internal arrays */
	void Reset();

	/** Array of root level windows, each containing a tree of widget nodes */
	TArray<TSharedPtr<FWidgetReflectorNodeBase>> Windows;

	/** Contains the simulation data entry for each entry  in Windows */
	TArray<FWidgetSnapshotNavigationSimulationData> NavigationSimulationData;

	/** Contains a texture data entry for each entry in Windows */
	TArray<FWidgetSnapshotTextureData> WindowTextureData;

	/** Contains a dynamic brush pointer for each entry in WindowTextureData */
	TArray<TSharedPtr<FSlateDynamicImageBrush>> WindowTextureBrushes;

	/** For the list, there must always be a valid array of simulation data. */
	FWidgetSnapshotNavigationSimulationData EmptyNavigationSimulationData;
};

/**
 * Visualizer to handle viewing and picking from a widget hierarchy snapshot
 */
class SWidgetSnapshotVisualizer : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnWidgetPathPicked, const TArray<TSharedRef<FWidgetReflectorNodeBase>>& /*PickedWidgetPath*/);
	DECLARE_DELEGATE_OneParam(FOnSnapshotWidgetAction, FWidgetReflectorNodeBase::TPointerAsInt);

	SLATE_BEGIN_ARGS(SWidgetSnapshotVisualizer)
		: _SnapshotData(nullptr)
		{}

		SLATE_ARGUMENT(const FWidgetSnapshotData*, SnapshotData)

		SLATE_EVENT(FOnWidgetPathPicked, OnWidgetPathPicked)
		SLATE_EVENT(FOnSnapshotWidgetAction, OnSnapshotWidgetSelected)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Called when the snapshot data we're observing is changed. This allows us to update our view. */
	void SnapshotDataUpdated();

	/** Called to update the list of selected widgets */
	void SetSelectedWidgets(const TArray<TSharedRef<FWidgetReflectorNodeBase>>& InSelectedWidgets);

	// SWidget interface
	virtual FReply OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	/** Called when the selected window is changed in the combo box. Used to update our view. */
	void OnWindowSelectionChanged(TSharedPtr<FWidgetReflectorNodeBase> InWindow, ESelectInfo::Type InReason);

	/** Get the combo item text to use for the given window */
	static FText GetWindowPickerComboItemText(TSharedPtr<FWidgetReflectorNodeBase> InWindow);

	/** Get the combo item text for the currently selected window */
	FText GetSelectedWindowComboItemText() const;

	/** Create a widget for the items in the window picker combo box */
	TSharedRef<SWidget> GenerateWindowPickerComboItem(TSharedPtr<FWidgetReflectorNodeBase> InWindow) const;

	/** Get the current text to use for the "Pick Snapshot Widget" button */
	FText GetPickWidgetText() const;

	/** Get the current check state to use for the "Pick Snapshot Widget" button */
	ECheckBoxState GetPickWidgetColor() const;

	/** Called when the "Pick Snapshot Widget" button is clicked */
	void OnPickWidgetClicked();

	/** Is there any snapshot data to save, visualized... */
	bool HasValidSnapshot() const;

	/** Should we show the Navigation Event Simulation List. */
	EVisibility HandleGetNavigationSimulationListVisibility() const;

#if SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM
	/** Called when the "Save Snapshot" button is clicked */
	void OnSaveSnapshotClicked();
#endif // SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM

private:
	/** Snapshot data we're visualizing */
	const FWidgetSnapshotData* SnapshotDataPtr;

	/** Window picker combo box */
	TSharedPtr<SComboBox<TSharedPtr<FWidgetReflectorNodeBase>>> WindowPickerCombo;

	/** Snapshot image */
	TSharedPtr<SScrollableSnapshotImage> SnapshotImage;

	/** Navigation Simulation list */
	TSharedPtr<SNavigationSimulationSnapshotList> NavigationSimulationList;
};
