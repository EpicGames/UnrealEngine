// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateNavigationEventSimulator.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

/** */
class INavigationEventSimulationView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(INavigationEventSimulationView) {}
	SLATE_END_ARGS()

	virtual void SetSimulationResult(TArray<FSlateNavigationEventSimulator::FSimulationResult> SimulationResult) = 0;
	virtual void SelectWidget(const TSharedPtr<SWidget>& Widget) = 0;
	virtual int32 PaintSimuationResult(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) = 0;
};
