// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class INavigationEventSimulationView;
class FSlateNavigationEventSimulator;
class SWidget;
class FWorkspaceItem;
class INavigationEventSimulationView;

DECLARE_DELEGATE_OneParam(FSimpleWidgetDelegate, TWeakPtr<const SWidget>);

struct FNavigationEventSimulationViewArgs
{
	FSimpleWidgetDelegate OnWidgetSelected;
	FSimpleWidgetDelegate OnNavigateToWidget;
};

/**
 * Interface for messaging modules.
 */
class ISlateReflectorModule
	: public IModuleInterface
{
public:

	/**
	 * Display the widget reflector, either spawned from a tab manager, or in a new window if the tab manager can't be used
	 */
	virtual void DisplayWidgetReflector() = 0;

	/**
	 * Display the texture atlas visualizer, either spawned from a tab manager, or in a new window if the tab manager can't be used
	 */
	virtual void DisplayTextureAtlasVisualizer() = 0;

	/**
	 * Display the texture atlas visualizer, either spawned from a tab manager, or in a new window if the tab manager can't be used
	 */
	virtual void DisplayFontAtlasVisualizer() = 0;

	/**
	 * Registers a tab spawner for the widget reflector.
	 *
	 * @param WorkspaceGroup The workspace group to insert the tab into.
	 */
	virtual void RegisterTabSpawner( const TSharedPtr<FWorkspaceItem>& WorkspaceGroup ) = 0;

	/** Unregisters the tab spawner for the widget reflector. */
	virtual void UnregisterTabSpawner() = 0;

	/** Get the Navigation Event Simulator used by the widget reflector. */
	virtual FSlateNavigationEventSimulator* GetNavigationEventSimulator() const = 0;

	/** Create a widget to visualize the result of a Navigation Event Simulation. */
	virtual TSharedRef<INavigationEventSimulationView> CreateNavigationEventSimulationView(const FNavigationEventSimulationViewArgs& DetailsViewArgs) = 0;

public:

	/** Virtual destructor. */
	virtual ~ISlateReflectorModule() = default;
};
