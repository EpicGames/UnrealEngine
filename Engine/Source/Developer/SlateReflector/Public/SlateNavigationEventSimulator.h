// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Events.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/WidgetPath.h"
#include "Misc/Optional.h"
#include "Types/SlateEnums.h"

#define UE_API SLATEREFLECTOR_API

class FNavigationReply;
class SWidget;
class SWindow;

/*
 * HOW TO DEBUG NAVIGATION EVENTS
 * Test the static result via FSlateNavigationEventSimulator.
 * If the simulation doesn't work, your widget is probably not be configured properly.
 * If the simulation does work, use the "WidgetReflector events log" or the "Console Slate Debugger" to get information on the navigation event while playing.
 *  A - If you do not find the event, check it has been consumed by the viewport or by a widget with OnKeyDown.
 *  B - If you find the event, note if the Reply and Consumed widget are different.
 *    i- If they are different, then use the "WidgetReflector routing" tool to understand why another widget has consumed the routing.
 *       For example, the widget can be disabled or it may have a custom OnNavigation event.
 *    ii- If they are not different, then the GameViewportClient or the application may have consumed the event.
 * You may also check if you received the focus event with the "WidgetReflector events log" or with the "Console Slate Debugger".
 */

/**
 * Simulate navigation attempt and collect the result for later display.
 * Some elements may steel the navigation attempt dynamically and can be evaluated until evaluated in game.
 * A list of elements that can still the navigation attempt and modify the result.
 * 1. The viewport may consume the Navigation.
 * 2. A widget may override the OnNavigation function and return a different result dynamically.
 * 3. A widget can change its "enabled" and "supports keyboard focus" flags dynamically during gameplay.
 * 4. The GameViewportClient may consume the event via the CustomNavigationEvent.
 * 5. The application or a widget can consume the "set focus" event. Resulting in different behavior.
 */
class FSlateNavigationEventSimulator
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(TOptional<FWidgetPath>, FHandleNavigation, uint32 /*UserIndex*/, const TSharedPtr<SWidget>& /*InDestination*/);

	enum class ENavigationStyle
	{
		FourCardinalDirections,
		ConceptualNextAndPrevious,
	};
	static UE_API FText ToText(ENavigationStyle NavigationStyle);

	enum class ERoutedReason
	{
		BoundaryRule,
		Window,
		LastWidget,
	};
	static UE_API FText ToText(ERoutedReason RoutedReason);

	struct FSimulatedReply
	{
		FSimulatedReply() = default;
		FSimulatedReply(const FNavigationReply&);

		TSharedPtr<SWidget> EventHandler;
		TSharedPtr<SWidget> FocusRecipient;
		EUINavigationRule BoundaryRule;
	};

	struct FSimulationResult
	{
		/** From where the navigation started. */
		FWidgetPath NavigationSource;

		/** The result of the navigation simulation. */
		FWidgetPath NavigationDestination;

		// The navigation type
		EUINavigation NavigationType = EUINavigation::Invalid;

		/** The reply of received from the routed widget. */
		FSimulatedReply NavigationReply;

		/** The reason we choose that boundary widget. */
		ERoutedReason RoutedReason = ERoutedReason::LastWidget;

		/** Base on the boundary rule, the widget that should receive the focus. */
		TSharedPtr<SWidget> WidgetThatShouldReceivedFocus;

		/** The widget that will received the focus. */
		TSharedPtr<SWidget> FocusedWidgetPath;

		/** Can not return a result since it would involve calling a dynamic callback. */
		bool bIsDynamic = false;
		/** The event is handle, even if the destination widget is nullptr or the Viewport->HandleNavigation return false. */
		bool bAlwaysHandleNavigationAttempt = false;
		/** Was able to find the widget to focus. */
		bool bCanFindWidgetForSetFocus = false;
		/** Does the routed widget has navigation meta data. */
		bool bRoutedHandlerHasNavigationMeta = false;
		/** Handled by the viewport. See OnViewportHandleNavigation. */
		bool bHandledByViewport = false;

		bool IsHandled() const { return NavigationDestination.IsValid() || bAlwaysHandleNavigationAttempt || bHandledByViewport; }
		bool IsValid() const { return NavigationType != EUINavigation::Invalid; }
	};

public:
	/**
	 * Simulate for each widgets that are enabled and support keyboard focus
	 */
	UE_API TArray<FSimulationResult> SimulateForEachWidgets(const FWidgetPath& WidgetPath, int32 UserIndex, ENavigationGenesis Genesis, ENavigationStyle Navigation);
	UE_API TArray<FSimulationResult> SimulateForEachWidgets(const TSharedRef<SWindow>& Window, int32 UserIndex, ENavigationGenesis Genesis, ENavigationStyle Navigation);
	UE_API TArray<FSimulationResult> SimulateForEachWidgets(const FWidgetPath& WidgetPath, int32 UserIndex, ENavigationGenesis Genesis, EUINavigation Navigation);
	UE_API TArray<FSimulationResult> SimulateForEachWidgets(const TSharedRef<SWindow>& Window, int32 UserIndex, ENavigationGenesis Genesis, EUINavigation Navigation);

	/**
	 * Simulate a navigation event for a widget
	 */
	UE_API TArray<FSimulationResult> Simulate(const FWidgetPath& WidgetPath, int32 UserIndex, ENavigationGenesis Genesis, ENavigationStyle Navigation);
	UE_API FSimulationResult Simulate(const FWidgetPath& WidgetPath, int32 UserIndex, ENavigationGenesis Genesis, EUINavigation Navigation);

public:
	/** Once the destination widget has been decided, the viewport may handled the event. */
	FHandleNavigation OnViewportHandleNavigation;

private:
	FSimulationResult InterpretResult(const FWidgetPath& WidgetPath, const FNavigationEvent& NavigationEvent, const FNavigationReply& Reply, const FArrangedWidget& BoundaryWidget);
};

#undef UE_API
