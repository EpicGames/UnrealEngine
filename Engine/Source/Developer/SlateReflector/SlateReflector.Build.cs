// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SlateReflector : ModuleRules
{
	public SlateReflector(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CoreUObject",
				"ApplicationCore",
				"InputCore",
				"Slate",
				"SlateCore",
				
				"Json",
				"AssetRegistry",
                "MessageLog",
				"ToolWidgets",
				"DesktopPlatform",
				"ImageCore",
				"ImageWrapper",
            }
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
                "Messaging",
				"MessagingCommon",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"Messaging",
			}
		);

		// Editor builds include SessionServices to populate the remote target drop-down for remote widget snapshots
		if (Target.bCompileAgainstEditor)
		{
			PublicDefinitions.Add("SLATE_REFLECTOR_HAS_SESSION_SERVICES=1");

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"PropertyEditor",
					"UnrealEd",
					"UMG",
				}
			);

            PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"SessionServices",
                }
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"SessionServices",
				}
			);
        }
		else
		{
			PublicDefinitions.Add("SLATE_REFLECTOR_HAS_SESSION_SERVICES=0");
		}

		// DesktopPlatform is only available for Editor and Program targets (running on a desktop platform)
		bool IsDesktopPlatformType = Target.Platform == UnrealBuildTool.UnrealTargetPlatform.Win64
			|| Target.Platform == UnrealBuildTool.UnrealTargetPlatform.Mac
			|| Target.Platform == UnrealBuildTool.UnrealTargetPlatform.Linux;
		if (Target.Type == TargetType.Editor || (Target.Type == TargetType.Program && IsDesktopPlatformType))
		{
			PublicDefinitions.Add("SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM=1");

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"DesktopPlatform",
				}
			);
		}
		else
		{
			PublicDefinitions.Add("SLATE_REFLECTOR_HAS_DESKTOP_PLATFORM=0");
		}

		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrecompileForTargets = PrecompileTargetsType.Any;
		}

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		bDisableAutoRTFMInstrumentation = true;
	}
}
